// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class CsvMetrics : ModuleRules
{
	public CsvMetrics(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		/**
		 * Add public dependencies that you statically link with here
		 *	NOTE: Please use the "PrivateDependencyModuleNames" list to add your module unless
		 *		it containing "types" that are required in this module's public headers.
		 */

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"CoreUObject",
				"WorldMetricsCore",
			}
		);

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"Engine",
			}
		);
	}
}
