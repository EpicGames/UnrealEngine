// Copyright Epic Games, Inc. All Rights Reserved.

#include "CsvActorCountMetric.h"

#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "WorldMetricsActorTracker.h"
#include "WorldMetricsSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CsvActorCountMetric)

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, ActorCount);

namespace UE::WorldMetrics::Private
{
static TAutoConsoleVariable<int32> CVarCsvRecordActorCountThreshold(
	TEXT("csv.RecordActorCountThreshold"),
	5,
	TEXT("Number of instances of a native Actor class required before recording to a CSV stat"));

}  // namespace UE::WorldMetrics::Private

//---------------------------------------------------------------------------------------------------------------------
// UCsvActorCountMetric
//---------------------------------------------------------------------------------------------------------------------

void UCsvActorCountMetric::Initialize()
{
	GetOwner().AcquireExtension<UWorldMetricsActorTracker>(this);
}

void UCsvActorCountMetric::Deinitialize()
{
	GetOwner().ReleaseExtension<UWorldMetricsActorTracker>(this);

	ActorClassNameCounter.Reset();
	TotalActorCount = 0;
}

SIZE_T UCsvActorCountMetric::GetAllocatedSize() const
{
	return ActorClassNameCounter.GetAllocatedSize();
}

int32 UCsvActorCountMetric::NumActors(const FName& NativeClassName) const
{
	if (const int32* ActorClassCount = ActorClassNameCounter.Find(NativeClassName))
	{
		return *ActorClassCount;
	}
	return 0;
}

void UCsvActorCountMetric::Update(float /*DeltaTimeInSeconds*/)
{
#if CSV_PROFILER_STATS
	if (FCsvProfiler::Get()->IsCapturing())
	{
		const int32 Threshold = UE::WorldMetrics::Private::CVarCsvRecordActorCountThreshold.GetValueOnAnyThread();
		for (const TPair<FName, int32>& ActorClassNameCount : ActorClassNameCounter)
		{
			if (ActorClassNameCount.Value > Threshold)
			{
				FCsvProfiler::Get()->RecordCustomStat(
					ActorClassNameCount.Key, CSV_CATEGORY_INDEX(ActorCount), ActorClassNameCount.Value,
					ECsvCustomStatOp::Set);
			}
		}
	}
#endif	// CSV_PROFILER_STATS
}

void UCsvActorCountMetric::OnActorAdded(const AActor* Actor)
{
	if (!Actor->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		const UClass* ParentNativeClass = GetParentNativeClass(Actor->GetClass());
		const FName NativeClassName = ParentNativeClass ? ParentNativeClass->GetFName() : NAME_None;
		int32& ActorClassCount = ActorClassNameCounter.FindOrAdd(NativeClassName);
		++ActorClassCount;
		++TotalActorCount;
	}
}

void UCsvActorCountMetric::OnActorRemoved(const AActor* Actor)
{
	if (!Actor->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		const UClass* ParentNativeClass = GetParentNativeClass(Actor->GetClass());
		const FName NativeClassName = ParentNativeClass ? ParentNativeClass->GetFName() : NAME_None;
		if (int32* ActorClassCount = ActorClassNameCounter.Find(NativeClassName))
		{
			--(*ActorClassCount);
		}
		--TotalActorCount;
	}
}
