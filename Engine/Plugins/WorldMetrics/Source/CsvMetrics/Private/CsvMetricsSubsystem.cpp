// Copyright Epic Games, Inc. All Rights Reserved.

#include "CsvMetricsSubsystem.h"

#include "Engine/World.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "WorldMetricsSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CsvMetricsSubsystem)

namespace UE::CsvMetrics::Private
{
bool CanHaveCsvMetrics(const UWorld* World)
{
#if CSV_PROFILER_STATS
	return World && World->IsGameWorld();
#else
	return false;
#endif	// CSV_PROFILER_STATS
}

}  // namespace UE::CsvMetrics::Private

//---------------------------------------------------------------------------------------------------------------------
// UCsvMetricsSubsystem
//---------------------------------------------------------------------------------------------------------------------

bool UCsvMetricsSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	// TODO: enable once CSVActorClassNameToCountMap is removed.
	return false;  // UE::CsvMetrics::Private::CanHaveCsvMetrics(Cast<UWorld>(Outer));
}

void UCsvMetricsSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	BindProfilerCallbacks();

	Super::Initialize(Collection);
}

void UCsvMetricsSubsystem::Deinitialize()
{
	UnbindProfilerCallbacks();

	Metrics.Reset();

	Super::Deinitialize();
}

void UCsvMetricsSubsystem::BindProfilerCallbacks()
{
#if CSV_PROFILER_STATS
	FCsvProfiler* CsvProfiler = FCsvProfiler::Get();
	if (ensure(CsvProfiler))
	{
		ProfileStartHandle = CsvProfiler->OnCSVProfileStart().AddUObject(this, &UCsvMetricsSubsystem::AddMetrics);

		ProfileEndHandle = CsvProfiler->OnCSVProfileEnd().AddUObject(this, &UCsvMetricsSubsystem::RemoveMetrics);
	}
#endif	// CSV_PROFILER_STATS
}

void UCsvMetricsSubsystem::UnbindProfilerCallbacks()
{
#if CSV_PROFILER_STATS
	FCsvProfiler* CsvProfiler = FCsvProfiler::Get();
	if (ensure(CsvProfiler))
	{
		CsvProfiler->OnCSVProfileStart().Remove(ProfileStartHandle);
		CsvProfiler->OnCSVProfileEnd().Remove(ProfileEndHandle);
	}
#endif	// CSV_PROFILER_STATS
}

void UCsvMetricsSubsystem::AddMetrics()
{
	UWorldMetricsSubsystem* WorldMetricsSubsystem = UWorldMetricsSubsystem::Get(GetWorld());
	if (ensure(WorldMetricsSubsystem))
	{
		check(Metrics.IsEmpty());
		for (const TSubclassOf<UWorldMetricInterface>& MetricClass : MetricClasses)
		{
			Metrics.Emplace(WorldMetricsSubsystem->AddMetric(MetricClass));
		}
	}
}

void UCsvMetricsSubsystem::RemoveMetrics()
{
	UWorldMetricsSubsystem* WorldMetricsSubsystem = UWorldMetricsSubsystem::Get(GetWorld());
	if (ensure(WorldMetricsSubsystem))
	{
		for (UWorldMetricInterface* Metric : Metrics)
		{
			WorldMetricsSubsystem->RemoveMetric(Metric);
		}
		Metrics.Reset();
	}
}
