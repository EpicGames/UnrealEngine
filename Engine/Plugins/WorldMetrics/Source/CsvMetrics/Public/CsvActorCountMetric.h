// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldMetricInterface.h"
#include "WorldMetricsActorTrackerSubscriber.h"

#include "CsvActorCountMetric.generated.h"

/**
 * World actor count metric
 *
 * This metrics provides a basic actor counter sorting the world's actors into categories.
 * On update it writes the actor counts to CSV if the CSV profiler is available.
 */
UCLASS()
class UCsvActorCountMetric : public UWorldMetricInterface, public IWorldMetricsActorTrackerSubscriber
{
	GENERATED_BODY()

	/** UWorldMetricInterface interface. */
	[[nodiscard]] CSVMETRICS_API virtual SIZE_T GetAllocatedSize() const override;

	[[nodiscard]] int32 NumActors() const
	{
		return TotalActorCount;
	}

	[[nodiscard]] int32 NumActors(const FName& NativeClassName) const;

private:
	/** Map of Actor class names to instance counts */
	TMap<FName, int32> ActorClassNameCounter;

	/** Count of total actors created */
	int32 TotalActorCount = 0;

	/** UWorldMetricInterface interface. */
	virtual void Initialize() override;
	virtual void Deinitialize() override;
	virtual void Update(float DeltaTimeInSeconds) override;

	/** IWorldMetricsActorTrackerSubscriber interface. */
	virtual void OnActorRemoved(const AActor* Actor) ;
	virtual void OnActorAdded(const AActor* Actor);
};
