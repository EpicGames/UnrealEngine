// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"

#include "CsvMetricsSubsystem.generated.h"

class UWorldMetricInterface;

/**
 * Csv metrics subsystem
 *
 * This subsystem registers/unregisters its CSV metric collection whenever a CSV profiler capture is performed.
 */
UCLASS(MinimalAPI, Config = Engine, DefaultConfig)
class UCsvMetricsSubsystem final : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin USubsystem Interface.
	CSVMETRICS_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	CSVMETRICS_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	CSVMETRICS_API virtual void Deinitialize() override;
	//~ End USubsystem Interface.

	/** The CSV metric collection to add/remove when a CSV profiler capture stats/ends. */
	UPROPERTY(Config)
	TArray<TSubclassOf<UWorldMetricInterface>> MetricClasses;

private:
	/**
	 * The collection of active metrics while a CSV profiler capture is running.
	 * We rely on the World Metrics Subsystem's ownership for these so we don't keep hard-references. */
	TArray<UWorldMetricInterface*> Metrics;

	FDelegateHandle ProfileStartHandle;
	FDelegateHandle ProfileEndHandle;

	/**
	 * Binds/Unbinds handles to OnCSVProfileStart and OnCSVProfileEnd in order register/unregister this system's CSV
	 * metric collection.
	 */
	void BindProfilerCallbacks();
	void UnbindProfilerCallbacks();

	void AddMetrics();
	void RemoveMetrics();
};
