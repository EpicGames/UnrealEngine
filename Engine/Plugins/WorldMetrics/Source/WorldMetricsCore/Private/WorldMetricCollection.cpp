// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldMetricCollection.h"

#include "Algo/IndexOf.h"
#include "Logging/StructuredLog.h"
#include "UObject/Interface.h"
#include "WorldMetricInterface.h"
#include "WorldMetricsLog.h"
#include "WorldMetricsSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldMetricCollection)

namespace UE::WorldMetrics::Private
{

UWorldMetricsSubsystem* GetSubsystem(const UObject* WorldContextObject)
{
	if (UNLIKELY(!WorldContextObject))
	{
		return nullptr;
	}

	UWorld* World = WorldContextObject->GetWorld();
	if (UNLIKELY(!World))
	{
		return nullptr;
	}

	return UWorldMetricsSubsystem::Get(World);
}

}  // namespace UE::WorldMetrics::Private

FWorldMetricCollection::~FWorldMetricCollection()
{
	Reset();
}

void FWorldMetricCollection::Initialize(UObject* InOuter)
{
	Reset();
	Subsystem = UE::WorldMetrics::Private::GetSubsystem(InOuter);
	if (!Subsystem.IsValid())
	{
		UE_LOGFMT(
			LogWorldMetrics, Warning,
			"[{Function}] Collections require the outer to provide a valid World Metrics Subsystem.", __FUNCTION__);
	}
}

void FWorldMetricCollection::Reset()
{
	if (IsEnabled())
	{
		Enable(false);
	}
	Metrics.Reset();
}

bool FWorldMetricCollection::Enable(bool bEnable)
{
	if (bEnable == IsEnabled())
	{
		return false;
	}

	UWorldMetricsSubsystem* SubsystemPtr = Subsystem.Get();
	if (!ensure(ValidateSubsystem(SubsystemPtr, __FUNCTION__)))
	{
		return false;
	}

	if (bEnable)
	{
		if (Metrics.IsEmpty())
		{
			return false;
		}
		for (UWorldMetricInterface* Metric : Metrics)
		{
			SubsystemPtr->AddMetric(Metric);
		}
	}
	else
	{
		for (UWorldMetricInterface* Metric : Metrics)
		{
			SubsystemPtr->RemoveMetric(Metric);
		}
	}
	bIsEnabled = bEnable;
	return true;
}

UWorldMetricInterface* FWorldMetricCollection::Get(const TSubclassOf<UWorldMetricInterface>& InMetricClass) const
{
	UClass* ClassPtr = InMetricClass.Get();
	if (UNLIKELY(!ValidateClass(ClassPtr, __FUNCTION__)))
	{
		return nullptr;
	}

	const int32 Index = GetMetricIndex(ClassPtr);
	if (Index == INDEX_NONE)
	{
		return nullptr;
	}

	return Metrics[Index];
}

bool FWorldMetricCollection::Contains(const TSubclassOf<UWorldMetricInterface>& InMetricClass) const
{
	UClass* ClassPtr = InMetricClass.Get();
	if (UNLIKELY(!ValidateClass(ClassPtr, __FUNCTION__)))
	{
		return false;
	}

	return GetMetricIndex(ClassPtr) != INDEX_NONE;
}

bool FWorldMetricCollection::Add(const TSubclassOf<UWorldMetricInterface>& InMetricClass)
{
	UWorldMetricsSubsystem* SubsystemPtr = Subsystem.Get();
	if (!ensure(ValidateSubsystem(SubsystemPtr, __FUNCTION__)))
	{
		return false;
	}

	UClass* ClassPtr = InMetricClass.Get();
	if (UNLIKELY(!ValidateClass(ClassPtr, __FUNCTION__)))
	{
		return false;
	}

	const int32 Index = GetMetricIndex(ClassPtr);
	if (Index != INDEX_NONE)
	{
		return false;
	}

	UWorldMetricInterface* Metric = SubsystemPtr->CreateMetric(ClassPtr);
	if (UNLIKELY(!Metric))
	{
		UE_LOGFMT(LogWorldMetrics, Log, "[{Function}] Unexpected null metric.", __FUNCTION__);
		return false;
	}

	Metrics.Emplace(Metric);
	if (IsEnabled())
	{
		SubsystemPtr->AddMetric(Metric);
	}
	return true;
}

UWorldMetricInterface* FWorldMetricCollection::GetOrAdd(const TSubclassOf<UWorldMetricInterface>& InMetricClass)
{
	UWorldMetricsSubsystem* SubsystemPtr = Subsystem.Get();
	if (!ensure(ValidateSubsystem(SubsystemPtr, __FUNCTION__)))
	{
		return nullptr;
	}

	UClass* ClassPtr = InMetricClass.Get();
	if (UNLIKELY(!ValidateClass(ClassPtr, __FUNCTION__)))
	{
		return nullptr;
	}

	const int32 Index = GetMetricIndex(ClassPtr);
	if (Index != INDEX_NONE)
	{
		return Metrics[Index];
	}

	UWorldMetricInterface* Metric = SubsystemPtr->CreateMetric(ClassPtr);
	if (UNLIKELY(!Metric))
	{
		UE_LOGFMT(LogWorldMetrics, Log, "[{Function}] Unexpected null metric.", __FUNCTION__);
		return nullptr;
	}

	Metrics.Emplace(Metric);
	if (IsEnabled())
	{
		SubsystemPtr->AddMetric(Metric);
	}
	return Metric;
}

bool FWorldMetricCollection::Remove(const TSubclassOf<UWorldMetricInterface>& InMetricClass)
{
	UWorldMetricsSubsystem* SubsystemPtr = Subsystem.Get();
	if (!ensure(ValidateSubsystem(SubsystemPtr, __FUNCTION__)))
	{
		return false;
	}

	UClass* ClassPtr = InMetricClass.Get();
	if (UNLIKELY(!ValidateClass(ClassPtr, __FUNCTION__)))
	{
		return false;
	}

	const int32 Index = GetMetricIndex(ClassPtr);
	if (Index == INDEX_NONE)
	{
		return false;
	}

	SubsystemPtr->RemoveMetric(Metrics[Index]);
	Metrics.RemoveAt(Index);
	if (Metrics.IsEmpty())
	{
		Enable(false);
	}
	return true;
}

int32 FWorldMetricCollection::Num() const
{
	return Metrics.Num();
}

bool FWorldMetricCollection::IsEmpty() const
{
	return Metrics.IsEmpty();
}

void FWorldMetricCollection::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddStableReferenceArray(&Metrics);
}

bool FWorldMetricCollection::ValidateSubsystem(
	const UWorldMetricsSubsystem* InSubsystem,
	const ANSICHAR* CallingFunctionName) const
{
	if (!InSubsystem)
	{
		UE_LOGFMT(
			LogWorldMetrics, Warning, "[{Function}] requires a valid World Metric's Subsystem.", CallingFunctionName);
		return false;
	}
	return true;
}

bool FWorldMetricCollection::ValidateClass(const UClass* InClass, const ANSICHAR* CallingFunctionName) const
{
	if (!InClass)
	{
		UE_LOGFMT(
			LogWorldMetrics, Warning, "[{Function}] Unexpected invalid metric class (class was null).",
			CallingFunctionName);
		return false;
	}
	if (InClass->HasAnyClassFlags(CLASS_Abstract))
	{
		UE_LOGFMT(
			LogWorldMetrics, Warning, "[{Function}] Unexpected invalid metric class ('{Class}' is abstract).",
			CallingFunctionName, InClass->GetPathName());
		return false;
	}
	return true;
}

int32 FWorldMetricCollection::GetMetricIndex(const UClass* InMetricClass) const
{
	return Algo::IndexOfByPredicate(
		Metrics, [InMetricClass](const UWorldMetricInterface* Metric) { return Metric->GetClass() == InMetricClass; });
}
