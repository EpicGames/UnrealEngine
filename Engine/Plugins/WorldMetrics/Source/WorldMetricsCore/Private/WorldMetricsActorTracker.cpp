// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldMetricsActorTracker.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "WorldMetricsActorTrackerSubscriber.h"
#include "WorldMetricsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldMetricsActorTracker)

//---------------------------------------------------------------------------------------------------------------------
// UWorldMetricsActorTracker
//---------------------------------------------------------------------------------------------------------------------

void UWorldMetricsActorTracker::Initialize()
{
	UE_LOGF(LogWorldMetrics, Log, "[%s]", __FUNCTION__);

	BindWorldCallbacks();
}

void UWorldMetricsActorTracker::Deinitialize()
{
	UE_LOGF(LogWorldMetrics, Log, "[%s]", __FUNCTION__);

	UnbindWorldCallbacks();

	Subscribers.Reset();
}

void UWorldMetricsActorTracker::BindWorldCallbacks()
{
	check(GetWorld());

	ActorAddedToWorldHandle = GetWorld()->AddOnPostRegisterAllActorComponentsHandler(
		FOnPostRegisterAllActorComponents::FDelegate::CreateLambda([this](AActor* Actor)
																   { NotifyOnActorAdded(Actor); }));

	ActorRemovedFromWorldHandle = GetWorld()->AddOnPreUnregisterAllActorComponentsHandler(
		FOnPreUnregisterAllActorComponents::FDelegate::CreateLambda([this](AActor* Actor)
																	{ NotifyOnActorRemoved(Actor); }));
}

void UWorldMetricsActorTracker::UnbindWorldCallbacks()
{
	check(GetWorld());

	GetWorld()->RemoveOnPostRegisterAllActorComponentsHandler(ActorAddedToWorldHandle);
	ActorAddedToWorldHandle.Reset();

	GetWorld()->RemoveOnPreUnregisterAllActorComponentsHandler(ActorRemovedFromWorldHandle);
	ActorRemovedFromWorldHandle.Reset();
}

SIZE_T UWorldMetricsActorTracker::GetAllocatedSize() const
{
	SIZE_T Result = Subscribers.GetAllocatedSize();
	return Result;
}

void UWorldMetricsActorTracker::OnAcquire(UObject* InOwner)
{
	if (IWorldMetricsActorTrackerSubscriber* Subscriber = Cast<IWorldMetricsActorTrackerSubscriber>(InOwner))
	{
		Subscribers.Emplace(Subscriber);
		NotifyExistingActors(Subscriber);
	}
}

void UWorldMetricsActorTracker::OnRelease(UObject* InOwner)
{
	if (IWorldMetricsActorTrackerSubscriber* Subscriber = Cast<IWorldMetricsActorTrackerSubscriber>(InOwner))
	{
		Subscribers.RemoveSwap(Subscriber);
	}
}

void UWorldMetricsActorTracker::NotifyExistingActors(IWorldMetricsActorTrackerSubscriber* Subscriber)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldMetricsActorTracker::NotifyExistingActors);

	check(Subscriber);

	UWorld* World = GetWorld();
	check(World);

	for (ULevel* Level : World->GetLevels())
	{
		for (const AActor* Actor : Level->Actors)
		{
			if (Actor && Actor->HasActorRegisteredAllComponents())
			{
				Subscriber->OnActorAdded(Actor);
			}
		}
	}
}

void UWorldMetricsActorTracker::NotifyOnActorAdded(const AActor* Actor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldMetricsActorTracker::NotifyOnActorAdded);

	for (IWorldMetricsActorTrackerSubscriber* Subscriber : Subscribers)
	{
		Subscriber->OnActorAdded(Actor);
	}
}

void UWorldMetricsActorTracker::NotifyOnActorRemoved(const AActor* Actor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldMetricsActorTracker::NotifyOnActorRemoved);

	for (IWorldMetricsActorTrackerSubscriber* Subscriber : Subscribers)
	{
		Subscriber->OnActorRemoved(Actor);
	}
}
