// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldMetricsLog.h"

DEFINE_LOG_CATEGORY(LogWorldMetrics);
