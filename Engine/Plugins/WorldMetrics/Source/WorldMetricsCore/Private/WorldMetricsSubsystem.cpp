// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldMetricsSubsystem.h"

#include "Algo/IndexOf.h"
#include "Engine/World.h"
#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "WorldMetricInterface.h"
#include "WorldMetricsExtension.h"
#include "WorldMetricsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldMetricsSubsystem)

//---------------------------------------------------------------------------------------------------------------------
// UWorldMetricsSubsystem
//---------------------------------------------------------------------------------------------------------------------

bool UWorldMetricsSubsystem::CanHaveWorldMetrics(const UWorld* World)
{
	return World && (World->IsGameWorld() || World->WorldType == EWorldType::Editor);
}

UWorldMetricsSubsystem* UWorldMetricsSubsystem::Get(const UWorld* World)
{
	if (LIKELY(World))
	{
		return World->GetSubsystem<UWorldMetricsSubsystem>();
	}
	return nullptr;
}

bool UWorldMetricsSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return CanHaveWorldMetrics(Cast<UWorld>(Outer));
}

void UWorldMetricsSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UE_LOGF(LogWorldMetrics, Verbose, "[%s]", __FUNCTION__);

	Super::Initialize(Collection);
}

void UWorldMetricsSubsystem::Deinitialize()
{
	UE_LOGF(LogWorldMetrics, Verbose, "[%s]", __FUNCTION__);

	Clear();

	Super::Deinitialize();
}

void UWorldMetricsSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UWorldMetricsSubsystem* This = CastChecked<UWorldMetricsSubsystem>(InThis);

	for (TPair<TSubclassOf<UWorldMetricsExtension>, FExtension>& ExtensionPair : This->Extensions)
	{
		Collector.AddReferencedObject(ExtensionPair.Key.GetGCPtr());
		Collector.AddReferencedObject(ExtensionPair.Value.Instance);
	}
}

void UWorldMetricsSubsystem::BeginDestroy()
{
	if (!ensure(!IsEnabled()))
	{
		UE_LOGF(LogWorldMetrics, Error, "[%s] Unexpected subsystem state: IsEnabled", __FUNCTION__);
		Clear();
	}
	Super::BeginDestroy();
}

void UWorldMetricsSubsystem::InitializeMetrics()
{
	for (UWorldMetricInterface* Metric : Metrics)
	{
		check(Metric);
		Metric->Initialize();
	}
	PendingWarmUpFrames = WarmUpFrames;
	if (!ensure(PendingWarmUpFrames >= 0))
	{
		UE_LOGF(
			LogWorldMetrics, Warning, "[%s] Invalid warm-up frames value: %d, reset to zero.", __FUNCTION__,
			PendingWarmUpFrames);
		PendingWarmUpFrames = 0;
	}
}

void UWorldMetricsSubsystem::DeinitializeMetrics()
{
	for (UWorldMetricInterface* Metric : Metrics)
	{
		check(Metric);
		Metric->Deinitialize();
	}
}

void UWorldMetricsSubsystem::Clear()
{
	RemoveAllMetrics();

	Extensions.Reset();

	UE_LOGF(LogWorldMetrics, Verbose, "[%s]", __FUNCTION__);
}

void UWorldMetricsSubsystem::Enable(bool bEnable)
{
	if (bEnable)
	{
		if (IsEnabled())
		{
			return;
		}

		if (!CanHaveWorldMetrics(GetWorld()))
		{
			return;
		}

		if (Metrics.IsEmpty())
		{
			return;
		}

		InitializeMetrics();

		if (!ensure(UpdateRateInSeconds >= 0.f))
		{
			UE_LOGF(
				LogWorldMetrics, Warning, "[%s] Invalid update rate value: %.02fs, reset to zero.", __FUNCTION__,
				UpdateRateInSeconds);
			UpdateRateInSeconds = 0.f;
		}

		/*
		 * Regarding CreateLambda.
		 *
		 * This subsystem uses an update ticker, although it's not a TickableWorldSubsystem. There are two reasons for
		 * this design decision:
		 *
		 * 1. The subsystem only requires an update ticker for metric updates. Due to the short-lived nature of these,
		 *    it makes sense to implement a finer-grain control over the ticker enabling/disabling mechanism and prevent
		 *    incurring an unnecessary cost to the game update thread whenever the subsystem has no metrics.
		 * 2. At the moment of this writing, the validity checks included in UObject delegates incur a significant cost,
		 *    negatively impacting the performance of metric updates. This subsystem exclusively owns world metrics and
		 *    extensions, and their lifetime is bound to the system's Initialize/Deinitialize methods. For this reason,
		 *    we can assume a raw delegate. Note, however, that UObject methods cannot be bound using CreateRaw; hence,
		 *    CreateLambda is used instead.
		 */
		UpdateTickerHandle = FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda(
				[this](float DeltaTimeInSeconds)
				{
					OnUpdate(DeltaTimeInSeconds);
					return true;
				}),
			UpdateRateInSeconds);
	}
	else if (IsEnabled())
	{
		FTSTicker::RemoveTicker(UpdateTickerHandle);
		UpdateTickerHandle.Reset();

		DeinitializeMetrics();
	}
}

void UWorldMetricsSubsystem::SetUpdateRateInSeconds(float InSeconds)
{
	if (!ensure(InSeconds >= 0.f))
	{
		UE_LOGF(
			LogWorldMetrics, Warning, "[%s] Invalid UpdateRateInSeconds input value %.04f: ignored.",
			__FUNCTION__, InSeconds);
		return;
	}

	if (UpdateRateInSeconds != InSeconds)
	{
		UpdateRateInSeconds = InSeconds;

		if (IsEnabled())
		{
			Enable(false);
			Enable(true);
		}
	}
}

int32 UWorldMetricsSubsystem::NumExtensions() const
{
	return Extensions.Num();
}

bool UWorldMetricsSubsystem::HasAnyExtension() const
{
	return !Extensions.IsEmpty();
}

int32 UWorldMetricsSubsystem::NumMetrics() const
{
	return Metrics.Num();
}

bool UWorldMetricsSubsystem::HasAnyMetric() const
{
	return !Metrics.IsEmpty();
}

UWorldMetricInterface* UWorldMetricsSubsystem::CreateMetric(const TSubclassOf<UWorldMetricInterface>& InMetricClass)
{
	if (UNLIKELY(!InMetricClass))
	{
		UE_LOGF(LogWorldMetrics, Warning, "[%s] Unexpected null metric class", __FUNCTION__);
		return nullptr;
	}

	if (InMetricClass->HasAnyClassFlags(CLASS_Abstract))
	{
		UE_LOGF(
			LogWorldMetrics, Warning, "[%s] Parameter metric class is abstract: %ls", __FUNCTION__,
			*InMetricClass->GetFName().ToString());
		return nullptr;
	}

	UWorldMetricInterface* Metric = NewObject<UWorldMetricInterface>(this, InMetricClass, NAME_None, RF_Transient);
	if (UNLIKELY(!Metric))
	{
		UE_LOGF(
			LogWorldMetrics, Error, "[%s] Failed to create metric of class: %ls", __FUNCTION__,
			*InMetricClass->GetFName().ToString());
	}
	return Metric;
}

bool UWorldMetricsSubsystem::ContainsMetric(UWorldMetricInterface* InMetric) const
{
	if (UNLIKELY(!InMetric))
	{
		UE_LOGF(LogWorldMetrics, Warning, "[%s] Unexpected null metric instance", __FUNCTION__);
		return false;
	}
	return Algo::IndexOf(Metrics, InMetric) != INDEX_NONE;
}

UWorldMetricInterface* UWorldMetricsSubsystem::AddMetric(const TSubclassOf<UWorldMetricInterface>& InMetricClass)
{
	UWorldMetricInterface* Metric = CreateMetric(InMetricClass);
	AddMetric(Metric);
	return Metric;
}

bool UWorldMetricsSubsystem::AddMetric(UWorldMetricInterface* InMetric)
{
	if (UNLIKELY(!InMetric))
	{
		UE_LOGF(LogWorldMetrics, Warning, "[%s] Unexpected null metric instance", __FUNCTION__);
		return false;
	}

	const int32 MetricIndex = Algo::IndexOf(Metrics, InMetric);
	if (MetricIndex != INDEX_NONE)
	{
		return false;
	}

	Metrics.Emplace(InMetric);
	if (IsEnabled())
	{
		InMetric->Initialize();
	}
	else
	{
		Enable(true);
	}

	UE_LOGF(
		LogWorldMetrics, Verbose, "[%s] Added metric of class %ls.", __FUNCTION__,
		*InMetric->GetClass()->GetFName().ToString());

	return true;
}

bool UWorldMetricsSubsystem::RemoveMetric(UWorldMetricInterface* InMetric)
{
	if (UNLIKELY(!InMetric))
	{
		UE_LOGF(LogWorldMetrics, Warning, "[%s] Unexpected null metric instance", __FUNCTION__);
		return false;
	}

	const int32 MetricIndex = Algo::IndexOf(Metrics, InMetric);
	if (MetricIndex == INDEX_NONE)
	{
		return false;
	}

	if (IsEnabled())
	{
		InMetric->Deinitialize();
		VerifyMetricReleasedAllExtensions(InMetric);
	}
	Metrics.RemoveAt(MetricIndex);
	if (Metrics.IsEmpty())
	{
		VerifyRemoveOrphanExtensions();
		Enable(false);
	}

	UE_LOGF(
		LogWorldMetrics, Verbose, "[%s] Removed metric of class %ls", __FUNCTION__,
		*InMetric->GetClass()->GetFName().ToString());

	return true;
}

void UWorldMetricsSubsystem::RemoveAllMetrics()
{
	for (int32 MetricIndex = 0; MetricIndex < Metrics.Num(); ++MetricIndex)
	{
		UWorldMetricInterface* Metric = Metrics[MetricIndex];
		if (IsEnabled())
		{
			Metric->Deinitialize();
			VerifyMetricReleasedAllExtensions(Metric);
			Metric->MarkAsGarbage();
		}
	}
	Metrics.Reset();
	VerifyRemoveOrphanExtensions();
	Enable(false);
}

void UWorldMetricsSubsystem::ForEachMetric(const TFunctionRef<bool(const UWorldMetricInterface*)>& Func) const
{
	for (const UWorldMetricInterface* Metric : Metrics)
	{
		if (!Func(Metric))
		{
			break;
		}
	}
}

void UWorldMetricsSubsystem::OnUpdate(float DeltaTimeInSeconds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldMetricsSubsystem::OnUpdate);

	if (PendingWarmUpFrames > 0)
	{
		--PendingWarmUpFrames;
		return;
	}

	for (UWorldMetricInterface* Metric : Metrics)
	{
		check(Metric);
		Metric->Update(DeltaTimeInSeconds);
	}
}

UWorldMetricsExtension* UWorldMetricsSubsystem::AcquireExtension(
	UWorldMetricInterface* InMetricOwner,
	const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	return AcquireExtensionInternal(InMetricOwner, InExtensionClass);
}

UWorldMetricsExtension* UWorldMetricsSubsystem::AcquireExtension(
	UWorldMetricsExtension* InExtensionOwner,
	const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	return AcquireExtensionInternal(InExtensionOwner, InExtensionClass);
}

bool UWorldMetricsSubsystem::ReleaseExtension(
	UWorldMetricInterface* InMetricOwner,
	const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	return ReleaseExtensionInternal(InMetricOwner, InExtensionClass);
}

bool UWorldMetricsSubsystem::ReleaseExtension(
	UWorldMetricsExtension* InExtensionOwner,
	const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	return ReleaseExtensionInternal(InExtensionOwner, InExtensionClass);
}

UWorldMetricsExtension* UWorldMetricsSubsystem::AcquireExtensionInternal(
	UObject* InOwner,
	const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	if (UNLIKELY(!InOwner))
	{
		UE_LOGF(LogWorldMetrics, Warning, "[%s] Unexpected invalid owner", __FUNCTION__);
		return nullptr;
	}

	if (UNLIKELY(!InExtensionClass))
	{
		UE_LOGF(LogWorldMetrics, Warning, "[%s] Unexpected null extension class", __FUNCTION__);
		return nullptr;
	}

	if (InExtensionClass->HasAnyClassFlags(CLASS_Abstract))
	{
		UE_LOGF(
			LogWorldMetrics, Warning, "[%s] Parameter extension class is abstract: %ls", __FUNCTION__,
			*InExtensionClass->GetFName().ToString());
		return nullptr;
	}

	UWorldMetricsExtension* Extension = AcquireExistingExtension(InOwner, InExtensionClass);
	if (!Extension)
	{
		Extension = AddExtension(InOwner, InExtensionClass);
	}

	if (LIKELY(Extension))
	{
		Extension->OnAcquire(InOwner);
	}
	return Extension;
}

UWorldMetricsExtension* UWorldMetricsSubsystem::AcquireExistingExtension(
	UObject* InOwner,
	const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	FExtension* Extension = Extensions.Find(InExtensionClass);
	if (!Extension)
	{
		return nullptr;
	}

	Extension->Owners.Emplace(InOwner);

	return Extension->Instance;
}

UWorldMetricsExtension* UWorldMetricsSubsystem::AddExtension(
	UObject* InOwner,
	const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	UWorldMetricsExtension* ExtensionInstance =
		NewObject<UWorldMetricsExtension>(this, InExtensionClass, NAME_None, RF_Transient);	
	if (UNLIKELY(!ExtensionInstance))
	{
		UE_LOGF(
			LogWorldMetrics, Error, "[%s] Failed to create extension of class: %ls", __FUNCTION__,
			*InExtensionClass->GetFName().ToString());
		return nullptr;
	}

	FExtension& Extension = Extensions.Add(InExtensionClass);
	Extension.Instance = ExtensionInstance;
	Extension.Owners.Emplace(InOwner);
	ExtensionInstance->Initialize();

	UE_LOGF(
		LogWorldMetrics, Verbose, "[%s] Added extension of class: %ls", __FUNCTION__,
		*InExtensionClass->GetFName().ToString());

	return ExtensionInstance;
}

bool UWorldMetricsSubsystem::ReleaseExtensionInternal(
	UObject* InOwner,
	const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	if (UNLIKELY(!InOwner))
	{
		return false;
	}

	if (UNLIKELY(!InExtensionClass))
	{
		UE_LOGF(LogWorldMetrics, Warning, "[%s] Unexpected null extension class", __FUNCTION__);
		return false;
	}

	FExtension* Extension = Extensions.Find(InExtensionClass);
	if (!Extension)
	{
		return false;
	}

	Extension->Instance->OnRelease(InOwner);
	if (UNLIKELY(!Extension->Owners.Remove(InOwner)))
	{
		UE_LOGF(
			LogWorldMetrics, Error, "[%s] Parameter object doesn't own this extension: %ls", __FUNCTION__,
			*InExtensionClass->GetFName().ToString());
	}

	TryRemoveExtension(InExtensionClass);

	return true;
}

bool UWorldMetricsSubsystem::TryRemoveExtension(const TSubclassOf<UWorldMetricsExtension>& InExtensionClass)
{
	FExtension* Extension = Extensions.Find(InExtensionClass);
	if (Extension && Extension->Owners.IsEmpty())
	{
		UWorldMetricsExtension* ExtensionInstance = Extension->Instance;
		ExtensionInstance->Deinitialize();
		ExtensionInstance->MarkAsGarbage();

		Extensions.Remove(InExtensionClass);
		UE_LOGF(
			LogWorldMetrics, Verbose, "[%s] Removed extension of class %ls", __FUNCTION__,
			*ExtensionInstance->GetClass()->GetFName().ToString());

		return true;
	}
	return false;
}

void UWorldMetricsSubsystem::VerifyMetricReleasedAllExtensions(UWorldMetricInterface* InMetric)
{
	TArray<TSubclassOf<UWorldMetricsExtension>, TInlineAllocator<DefaultExtensionCapacity>> StaleExtensionClasses;
	for (TPair<TSubclassOf<UWorldMetricsExtension>, FExtension>& ExtensionPair : Extensions)
	{
		if (UNLIKELY(ExtensionPair.Value.Owners.Remove(InMetric)))
		{
			StaleExtensionClasses.Emplace(ExtensionPair.Key);
			UWorldMetricsExtension* Extension = ExtensionPair.Value.Instance;
			UE_LOGF(
				LogWorldMetrics, Warning, "[%s] World metric %ls did not release extension %ls in Deinitialize",
				__FUNCTION__, *InMetric->GetFName().ToString(), *Extension->GetName());
		}
	}

	for (const TSubclassOf<UWorldMetricsExtension>& ExtensionClass : StaleExtensionClasses)
	{
		TryRemoveExtension(ExtensionClass);
	}
}

void UWorldMetricsSubsystem::VerifyRemoveOrphanExtensions()
{
	if (Metrics.IsEmpty())
	{
		if (UNLIKELY(!Extensions.IsEmpty()))
		{
			TArray<TSubclassOf<UWorldMetricsExtension>, TInlineAllocator<DefaultExtensionCapacity>> ExtensionsToRemove;
			Extensions.GenerateKeyArray(ExtensionsToRemove);
			for (const TSubclassOf<UWorldMetricsExtension>& ExtensionClass : ExtensionsToRemove)
			{
				if (const FExtension* Extension = Extensions.Find(ExtensionClass))
				{
					UWorldMetricsExtension* ExtensionInstance = Extension->Instance;
					check(ExtensionInstance);

					UE_LOGF(
						LogWorldMetrics, Warning,
						"[%s] World extension %ls was acquired by another extension but never released.",
						__FUNCTION__, *ExtensionInstance->GetName());

					ExtensionInstance->Deinitialize();
					ExtensionInstance->MarkAsGarbage();
					Extensions.Remove(ExtensionClass);
				}
			}
			Extensions.Reset();
		}
	}
}
