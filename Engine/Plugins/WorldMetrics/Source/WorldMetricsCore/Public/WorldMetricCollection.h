// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Casts.h"
#include "Templates/Invoke.h"
#include "UObject/ObjectPtr.h"

#include "WorldMetricCollection.generated.h"

class UWorldMetricInterface;
class UWorldMetricsSubsystem;
template <typename T>
class TSubclassOf;
class FReferenceCollector;

/**
 * World metric's collection

 * A class representing a collection of world metrics. This class allows instantiating and running an arbitrary set of
 * metrics together. The collection observes one instance of per metric class and depends on the World Metric's
 * Subsystem to run the metrics. For this reason, the collection requires initialization. Users are responsible for
 * ensuring both their lifetime and that of the World Metric's Subsystem in their world object.
 */
USTRUCT()
struct FWorldMetricCollection
{
	GENERATED_BODY()

public:
	FWorldMetricCollection() = default;
	WORLDMETRICSCORE_API ~FWorldMetricCollection();

	[[nodiscard]] const UWorldMetricInterface* operator[](int32 Index) const
	{
		return Metrics[Index].Get();
	}

	[[nodiscard]] UWorldMetricInterface* operator[](int32 Index)
	{
		return Metrics[Index].Get();
	}

	/**
	 * Initializes the collection. Initializing a container with a valid object resets the current collection.
	 * @param InOuter: the object that must provide a valid World Metrics Subsystem through its world object.
	 */
	WORLDMETRICSCORE_API void Initialize(UObject* InOuter);

	/**
	 * Enables or disables the collection. When enabled, the collection adds all contained metrics to the World Metric
	 * Subsystem. When disabled, all contained metrics are removed from it.
	 *
	 * @param bEnable True for enabling the collection and false otherwise.
	 * @return True if the enable state was changed to reflect the desired state.
	 */
	WORLDMETRICSCORE_API bool Enable(bool bEnable);

	/**
	 * @return True if the collection is enabled, and thus the contained metrics are running, or false otherwise.
	 */
	[[nodiscard]] bool IsEnabled() const
	{
		return bIsEnabled;
	}

	/**
	 * Gets the metric instance from the parameter class if it has been previously added.
	 *
	 * @param InMetricClass The class of the metric to be retrieved.
	 * @return A valid pointer to the metric if found, and nullptr otherwise.
	 */
	[[nodiscard]] WORLDMETRICSCORE_API UWorldMetricInterface* Get(
		const TSubclassOf<UWorldMetricInterface>& InMetricClass) const;

	/**
	 * Gets the metric instance from the type parameter class if it has been previously added.
	 *
	 * @tparam MetricType The class of the metric to be retrieved. Constrained to be a class derived from
	 * UWorldMetricInterface.
	 * @return A valid pointer to the metric if found, and nullptr otherwise.
	 */
	template <typename MetricType>
		requires(std::is_base_of_v<UWorldMetricInterface, MetricType>)
	[[nodiscard]] MetricType* Get() const
	{
		return static_cast<MetricType*>(Get(MetricType::StaticClass()));
	}

	/**
	 * Checks if the parameter metric class has been added.
	 *
	 * @param InMetricClass The class of the metric to be retrieved.
	 * @return True if the parameter metric class is registered and false otherwise.
	 */
	[[nodiscard]] WORLDMETRICSCORE_API bool Contains(const TSubclassOf<UWorldMetricInterface>& InMetricClass) const;

	/**
	 * Checks if the type parameter metric class has been added.
	 *
	 * @tparam MetricType The UWorldMetricInterface's subclass of the metric to query.
	 * @return True if the parameter metric class is registered and false otherwise.
	 */
	template <typename MetricType>
		requires(std::is_base_of_v<UWorldMetricInterface, MetricType>)
	[[nodiscard]] bool Contains() const
	{
		return Contains(MetricType::StaticClass());
	}

	/**
	 * Adds a metric instance of the parameter class unless a metric of the same class already exists.
	 *
	 * @param InMetricClass The class of the metric to add.
	 * @return True if a metric of the parameter metric class is added and false otherwise.
	 */
	WORLDMETRICSCORE_API bool Add(const TSubclassOf<UWorldMetricInterface>& InMetricClass);

	template <typename MetricType>
		requires(std::is_base_of_v<UWorldMetricInterface, MetricType>)
	bool Add()
	{
		return Add(MetricType::StaticClass());
	}

	/**
	 * Gets the metric instance of the parameter class or adds a new one if missing.
	 *
	 * @param InMetricClass The class of the metric to add or get.
	 * @return The added or previously existing instance of the parameter class.
	 */
	[[nodiscard]] WORLDMETRICSCORE_API UWorldMetricInterface* GetOrAdd(const TSubclassOf<UWorldMetricInterface>& InMetricClass);

	template <typename MetricType>
		requires(std::is_base_of_v<UWorldMetricInterface, MetricType>)
	[[nodiscard]] MetricType* GetOrAdd()
	{
		return static_cast<MetricType*>(GetOrAdd(MetricType::StaticClass()));
	}

	/**
	 * Removes the metric matching the parameter class from the collection.
	 * The metric object gets automatically garbage-collected unless another system holds a hard-reference to it.
	 *
	 * @param InMetricClass The class corresponding to the metric to remove.
	 * @return True if a metric matching the parameter class is removed and false otherwise.
	 */
	WORLDMETRICSCORE_API bool Remove(const TSubclassOf<UWorldMetricInterface>& InMetricClass);

	/**
	 * Removes the metric matching the type parameter class from the collection.
	 * The metric object gets automatically garbage-collected unless another system holds a hard-reference to it.
	 *
	 * @tparam MetricType The UWorldMetricInterface's subclass of the metric to remove.
	 * @return True if a metric matching the type parameter class is removed and false otherwise.
	 */
	template <typename MetricType>
		requires(std::is_base_of_v<UWorldMetricInterface, MetricType>)
	bool Remove()
	{
		return Remove(MetricType::StaticClass());
	}

	/**
	 * Invokes the parameter function on each of the metrics contained by the collection.
	 * @param Func The function which will be invoked for each metric. The function should return true to continue
	 * execution, or false otherwise.
	 */
	template <typename FuncType>
		requires(std::is_invocable_r_v<bool, std::decay_t<FuncType>, const UWorldMetricInterface*>)
	void ForEach(FuncType&& Func) const
	{
		for (const UWorldMetricInterface* Metric : Metrics)
		{
			if (!Invoke(Func, Metric))
			{
				break;
			}
		}
	}

	template <typename FuncType>
		requires(std::is_invocable_r_v<bool, std::decay_t<FuncType>, UWorldMetricInterface*>)
	void ForEach(FuncType&& Func)
	{
		for (UWorldMetricInterface* Metric : Metrics)
		{
			if (!Invoke(Func, Metric))
			{
				break;
			}
		}
	}

	/**
	 * Invokes the parameter function on each of the metrics of the specified type contained by the collection.
	 * @param Func The function which will be invoked for each metric of the specified type. The function should return
	 * true to continue execution, or false otherwise.
	 */
	template <typename MetricType, typename FuncType>
		requires(
			std::is_base_of_v<UWorldMetricInterface, MetricType> &&
			std::is_invocable_r_v<bool, std::decay_t<FuncType>, const MetricType*>)
	void ForEach(FuncType&& Func) const
	{
		for (UWorldMetricInterface* Metric : Metrics)
		{
			if (const MetricType* Derived = Cast<MetricType>(Metric))
			{
				if (!Invoke(Func, Derived))
				{
					break;
				}
			}
		}
	}

	template <typename MetricType, typename FuncType>
		requires(
			std::is_base_of_v<UWorldMetricInterface, MetricType> &&
			std::is_invocable_r_v<bool, std::decay_t<FuncType>, MetricType*>)
	void ForEach(FuncType&& Func)
	{
		for (UWorldMetricInterface* Metric : Metrics)
		{
			if (MetricType* Derived = Cast<MetricType>(Metric))
			{
				if (!Invoke(Func, Derived))
				{
					break;
				}
			}
		}
	}

	/**
	 * Removes all contained metrics from the World Metric Subsystem, and removes it from the collection.
	 */
	WORLDMETRICSCORE_API void Reset();

	/**
	 * @return The number of metrics currently contained by the collection.
	 */
	[[nodiscard]] WORLDMETRICSCORE_API int32 Num() const;

	/**
	 * @return True if the collection is empty and contains no metrics.
	 */
	[[nodiscard]] WORLDMETRICSCORE_API bool IsEmpty() const;

	/**
	 * Collects the references held by this collection. Use this method for non UPROPERTY collection instances to
	 * prevent the metric objects from being GC.
	 * @param Collector the destination reference collector.
	 */
	WORLDMETRICSCORE_API void AddReferencedObjects(FReferenceCollector& Collector);

private:
	/* List of metrics objects.*/
	UPROPERTY(Transient)
	TArray<TObjectPtr<UWorldMetricInterface>> Metrics;

	/* World Metric's Subsystem provided by the outer object. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UWorldMetricsSubsystem> Subsystem;

	/* Flag indicating whether the contained metrics are enabled and running in the World Metric's Subsystem. */
	UPROPERTY(Transient)
	bool bIsEnabled = false;

	/**
	 * @return True if the subsystem is valid.
	 */
	[[nodiscard]] bool ValidateSubsystem(const UWorldMetricsSubsystem* InSubsystem, const ANSICHAR* CallingFunctionName)
		const;

	/**
	 * @return True if the class is valid and complete.
	 */
	[[nodiscard]] bool ValidateClass(const UClass* InClass, const ANSICHAR* CallingFunctionName) const;

	/**
	 * Returns the live world metric list index corresponding to the parameter metric class.
	 *
	 * @param InMetricClass: the class type of the world metric.
	 * @return a valid index value or INDEX_NONE if not matching metric exists.
	 */
	[[nodiscard]] int32 GetMetricIndex(const UClass* InMetricClass) const;
};
