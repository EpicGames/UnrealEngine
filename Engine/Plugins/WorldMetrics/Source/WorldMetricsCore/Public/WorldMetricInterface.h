// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "WorldMetricInterface.generated.h"

/**
 * World metric's interface
 *
 * This is the required interface class to implement world metrics.
 */
UCLASS(abstract, MinimalAPI, Within=WorldMetricsSubsystem)
class UWorldMetricInterface : public UObject
{
	GENERATED_BODY()

public:
	/*
	 * Returns the amount of memory allocated by this class, not including sizeof(*this).
	 * @return the result size in bytes.
	 */
	[[nodiscard]] virtual SIZE_T GetAllocatedSize() const
		PURE_VIRTUAL(GetAllocatedSize, return 0;);

	/*
	 * Returns the owning World Metrics Subsystem which is expected to be valid during this object's lifetime.
	 * @return a reference to the owning World Metrics Subsystem.
	 */
	[[nodiscard]] UWorldMetricsSubsystem& GetOwner() const
	{
		return *GetOuterUWorldMetricsSubsystem();
	}

private:
	/**
	 * Metric initialization method called by the World Metrics subsystem on metric addition.
	 */
	virtual void Initialize()
	{
	}

	/**
	 * Metric initialization method called by the World Metrics subsystem before metric removal.
	 */
	virtual void Deinitialize()
	{
	}

	/**
	 * The metric's update method called by the World Metrics subsystem when enabled.
	 * @param DeltaTimeInSeconds the delta time in seconds from the last update.
	 */
	virtual void Update(float DeltaTimeInSeconds) PURE_VIRTUAL(Update, );

	friend class UWorldMetricsSubsystem;
};
