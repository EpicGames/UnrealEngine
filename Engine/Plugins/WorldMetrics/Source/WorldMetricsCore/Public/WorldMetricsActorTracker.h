// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldMetricsExtension.h"

#include "WorldMetricsActorTracker.generated.h"

class IWorldMetricsActorTrackerSubscriber;

/**
 * Actor tracker class
 *
 * This class provides a tracking context for actors added and removed from the world. For an actor to be considered
 * added to the world, it must have all its components registered. This class provides two ways of usage:
 * - Polling mode: accessible through ForEachActorInWorld.
 * - Subscriber mode: this mode is enabled through acquire/release semantics. When acquired, the actor tracker
 * subscribes to the world's PostRegisterAllActorComponents and PreUnregisterAllActorComponentsRegistered events
 * tracking all unique actors. In addition, metrics implementing IWorldMetricsActorTrackerSubscriber receive actor
 * addition and removal notifications.
 */
UCLASS(MinimalAPI)
class UWorldMetricsActorTracker final : public UWorldMetricsExtension
{
	GENERATED_BODY()

public:
	/** UWorldMetricsExtension */
	[[nodiscard]] WORLDMETRICSCORE_API virtual SIZE_T GetAllocatedSize() const override;

private:
	FDelegateHandle ActorAddedToWorldHandle;
	FDelegateHandle ActorRemovedFromWorldHandle;

	/** Subscriber list. This list uses raw IWorldMetricsActorTrackerSubscriber pointers are used by design instead of
	 * TWeakInterfacePtr for performance purposes. This decision is possible because the UWorldMetricsSubsystem is the
	 * owner of both this extension and all its subscribers, guaranteeing the lifetime requirements. 
	 */
	TArray<IWorldMetricsActorTrackerSubscriber*, TInlineAllocator<16>> Subscribers;
	bool bIsEnabled = false;

	/** UWorldMetricsExtension */
	virtual void Initialize() override;
	virtual void Deinitialize() override;
	virtual void OnAcquire(UObject* InOwner) override;
	virtual void OnRelease(UObject* InOwner) override;

	/**
	 * Binds/Unbinds handles to PostRegisterAllActorComponents and PreUnregisterAllActorComponents in order to account
	 * for actors being added/removed to/from the world.
	 */
	void BindWorldCallbacks();
	void UnbindWorldCallbacks();

	/*
	 * Notifies the parameter subscriber the addition of all actors in the world with all their components registered.
	 * @param Subscriber: the subscriber to notify.
	 */
	void NotifyExistingActors(IWorldMetricsActorTrackerSubscriber* Subscriber);

	/**
	 * Fires an addition notification event to all subscribers implementing IWorldMetricsActorTrackerSubscriber.
	 * @param the actor to notify.
	 */
	void NotifyOnActorAdded(const AActor* Actor);

	/**
	 * Fires a removal notification event to all subscribers implementing IWorldMetricsActorTrackerSubscriber.
	 * @param the actor to notify.
	 */
	void NotifyOnActorRemoved(const AActor* Actor);
};
