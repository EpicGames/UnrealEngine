// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "WorldMetricsActorTrackerSubscriber.generated.h"

UINTERFACE(MinimalAPI)
class UWorldMetricsActorTrackerSubscriber : public UInterface
{
	GENERATED_BODY()
};

/**
 * Actor tracker subscriber interface
 *
 * This interface enables the implementing object class to subscribe to UWorldMetricsActorTracker and receive
 * notifications whenever an actor is added and removed from the world.
 */
class IWorldMetricsActorTrackerSubscriber
{
	GENERATED_BODY()

public:
	/**
	 * Triggers whenever a tracked actor is added.
	 * @param Actor: the actor pointer which is guaranteed to be valid until its corresponding OnActorRemoved event.
	 */
	virtual void OnActorAdded(const AActor* Actor) = 0;

	/**
	 * Triggers whenever a tracked actor is about to be removed.
	 * @param Actor: the actor pointer will no longer be valid after this event.
	 */
	virtual void OnActorRemoved(const AActor* Actor) = 0;
};
