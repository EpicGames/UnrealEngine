// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "WorldMetricsExtension.generated.h"

/**
 * Base class for an extension for the World Metrics subsystem.
 *
 * World Metrics subsystem extensions provide custom data/functionality to world metrics and other extensions (one to
 * many) and have the following characteristics:
 * 1. Unique: there can only be a single instance of each extension class.
 * 2. Exclusively managed and owned by the  World Metrics subsystem.. These should never be created directly.
 * 3. Implement acquired/release semantics. The World Metrics subsystem automatically initializes extensions on
 * acquisition and deinitializes them on release. The subsystem may deallocate an extension that have no acquisitions.
 */
UCLASS(abstract, MinimalAPI, Within=WorldMetricsSubsystem)
class UWorldMetricsExtension : public UObject
{
	GENERATED_BODY()

public:
	/*
	 * Returns the amount of memory allocated by this class, not including sizeof(*this).
	 * @return the result size in bytes.
	 */
	[[nodiscard]] virtual SIZE_T GetAllocatedSize() const
		PURE_VIRTUAL(GetAllocatedSize, return 0;);

	/*
	 * Returns the owning World Metrics Subsystem which is expected to be valid during this object's lifetime.
	 * @return a reference to the owning World Metrics Subsystem.
	 */
	[[nodiscard]] UWorldMetricsSubsystem& GetOwner() const
	{
		return *GetOuterUWorldMetricsSubsystem();
	}

private:
	/**
	 * Called by the World Metrics subsystem whenever a new extension is added.
	 */
	virtual void Initialize()
	{
	}

	/**
	 * Called by the World Metrics subsystem whenever an extension is about to be removed.
	 */
	virtual void Deinitialize()
	{
	}

	/* Called by the World Metrics subsystem whenever an extension is acquired. */
	virtual void OnAcquire(UObject* InOwner)
	{
	}

	/* Called by the World Metrics subsystem whenever an extension is released. */
	virtual void OnRelease(UObject* InOwner)
	{
	}

	friend class UWorldMetricsSubsystem;
};
