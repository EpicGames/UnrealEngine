// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Subsystems/WorldSubsystem.h"

#include "WorldMetricsSubsystem.generated.h"

class UWorldMetricInterface;
class UWorldMetricsExtension;
class UWorldMetricsSubsystem;

/**
 * World metrics subsystem
 *
 * This subsystem provides an interface to add and remove world metrics implementing the UWorldMetricInterface class.
 *
 * - Added metrics get automatically updated by the subsystem's ticker.
 * - The subsystem becomes an owner of all added metrics. The user is responsible for removing them when no longer
 *   needed so they can be garbage collected.
 * - Metrics can have extensions to add shared functionality.
 * - Extensions implement the UWorldMetricsExtension class and use Acquire/Release semantics. They can be acquired by
 *   either metrics or extensions. Initialization and deinitialization are the ideal phases to do so.
 * - The subsystem solely owns extensions and can automatically remove them for garbage collection whenever they are no
 *   longer acquired by any metric or extension.
 */
UCLASS(MinimalAPI, Config = Engine, DefaultConfig)
class UWorldMetricsSubsystem final : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	[[nodiscard]] WORLDMETRICSCORE_API static bool CanHaveWorldMetrics(const UWorld* World);
	[[nodiscard]] WORLDMETRICSCORE_API static UWorldMetricsSubsystem* Get(const UWorld* World);

	//~ Begin USubsystem
	WORLDMETRICSCORE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	WORLDMETRICSCORE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	WORLDMETRICSCORE_API virtual void Deinitialize() override;
	//~ End USubsystem

	//~ Begin UObject
	WORLDMETRICSCORE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	WORLDMETRICSCORE_API virtual void BeginDestroy() override;
	//~ End UObject

	[[nodiscard]] bool IsEnabled() const
	{
		return UpdateTickerHandle.IsValid();
	}

	/**
	 * Sets the subsystem update ticker rate in seconds. This method automatically restarts the subsystem if it's
	 * already enabled. Changing the update rate value on an enabled subsystem causes all metrics to be reinitialized.
	 *
	 * @requirement: parameter InSeconds value must be equal or greater than zero.
	 * @param InSeconds: the update rate value in seconds.
	 */
	WORLDMETRICSCORE_API void SetUpdateRateInSeconds(float InSeconds);

	/**
	 * @return the number of metrics currently contained by the subsystem.
	 */
	WORLDMETRICSCORE_API int32 NumMetrics() const;

	/**
	 * @return true if the subsystem contains any metric.
	 */
	WORLDMETRICSCORE_API bool HasAnyMetric() const;

	/**
	 * @return the number of extensions currently contained by the subsystem.
	 */
	WORLDMETRICSCORE_API int32 NumExtensions() const;

	/**
	 * @return true if the subsystem contains any extension.
	 */
	WORLDMETRICSCORE_API bool HasAnyExtension() const;

	/**
	 * Factory method to create world metric instances.
	 *
	 * @param InMetricClass: the class of the world metric to be retrieved.
	 * @return a valid pointer to a world metric if successfully created or nullptr otherwise.
	 */
	[[nodiscard]] WORLDMETRICSCORE_API UWorldMetricInterface* CreateMetric(
		const TSubclassOf<UWorldMetricInterface>& InMetricClass);

	template <typename MetricClass UE_REQUIRES(std::is_base_of_v<UWorldMetricInterface, MetricClass>)>
	[[nodiscard]] MetricClass* CreateMetric()
	{
		return static_cast<MetricClass*>(CreateMetric(MetricClass::StaticClass()));
	}

	/**
	 * @param InMetric: the metric instance to search.
	 * @return true if the parameter metric has been added or false otherwise.
	 */
	WORLDMETRICSCORE_API bool ContainsMetric(UWorldMetricInterface* InMetric) const;

	/**
	 * Adds a new metric instance of the parameter class to the subsystem.
	 *
	 * @param InMetricClass: the class of the metric to add.
	 * @return true if a metric of the parameter metric class is added and false otherwise.
	 */
	[[nodiscard]] WORLDMETRICSCORE_API UWorldMetricInterface* AddMetric(
		const TSubclassOf<UWorldMetricInterface>& InMetricClass);

	template <typename MetricClass UE_REQUIRES(std::is_base_of_v<UWorldMetricInterface, MetricClass>)>
	[[nodiscard]] MetricClass* AddMetric()
	{
		return static_cast<MetricClass*>(AddMetric(MetricClass::StaticClass()));
	}

	/**
	 * Adds the parameter metric instance to the subsystem. The subsystem will hold a hard-reference to the added
	 * metric. The user is responsible for removing it when no longer needed so it can be garbage collected.
	 *
	 * @param InMetric: the metric instance to add.
	 * @return true if the parameter metric is valid and it wasn't previously added or false otherwise.
	 */
	WORLDMETRICSCORE_API bool AddMetric(UWorldMetricInterface* InMetric);

	/**
	 * Removes the parameter metric instance if it was previously added. The subsystem releases the hard-reference of
	 * the metric removed so it can be garbage-collected.
	 *
	 * @param InMetric: the metric instance to remove.
	 * @return true if the parameter metric is removed or false otherwise.
	 */
	WORLDMETRICSCORE_API bool RemoveMetric(UWorldMetricInterface* InMetric);

	/**
	 * Const iteration method for each of the metrics added to the subsystem.
	 * @param Func The function which will be invoked for each metric. The function should return true to continue
	 * execution, or false otherwise.
	 */
	WORLDMETRICSCORE_API void ForEachMetric(const TFunctionRef<bool(const UWorldMetricInterface*)>& Func) const;

	/**
	 * Const iteration method for each of the metrics added to the subsystem of the template argument class.
	 * @param Func The function which will be invoked for each metric. The function should return true to continue
	 * execution, or false otherwise.
	 */
	template <typename MetricClass UE_REQUIRES(std::is_base_of_v<UWorldMetricInterface, MetricClass>)>
	void ForEachMetricOfClass(const TFunctionRef<bool(const MetricClass*)>& Func) const
	{
		for (const UWorldMetricInterface* Metric : Metrics)
		{
			if (const MetricClass* TypedMetric = Cast<MetricClass>(Metric))
			{
				if (!Func(TypedMetric))
				{
					break;
				}
			}
		}
	}

	/**
	 * Acquires an extension on behalf of a object. The extension will be created the first time it is acquired,
	 * and conversely it will be disabled once released by all owners.
	 *
	 * @param InOwner: The object acquiring the extension.
	 * @param InExtensionClass: The class of the desired extension.
	 * @return A pointer to the extension. The lifetime of the extension is managed by the subsystem.
	 */
	WORLDMETRICSCORE_API UWorldMetricsExtension* AcquireExtension(
		UWorldMetricInterface* InOwner,
		const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	template <class ExtensionClass UE_REQUIRES(std::is_base_of_v<UWorldMetricsExtension, ExtensionClass>)>
	ExtensionClass* AcquireExtension(UWorldMetricInterface* InMetricOwner)
	{
		return static_cast<ExtensionClass*>(AcquireExtension(InMetricOwner, ExtensionClass::StaticClass()));
	}

	WORLDMETRICSCORE_API UWorldMetricsExtension* AcquireExtension(
		UWorldMetricsExtension* InOwner,
		const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	template <class ExtensionClass UE_REQUIRES(std::is_base_of_v<UWorldMetricsExtension, ExtensionClass>)>
	ExtensionClass* AcquireExtension(UWorldMetricsExtension* InExtensionOwner)
	{
		return static_cast<ExtensionClass*>(AcquireExtension(InExtensionOwner, ExtensionClass::StaticClass()));
	}

	/**
	 * Releases a metric's ownership of an extension. If this was the last reference the extension will be disabled.
	 *
	 * @param InOwner: The object whose ownership should be released.
	 * @param InExtensionClass: The class type of the extension.
	 */
	WORLDMETRICSCORE_API bool ReleaseExtension(
		UWorldMetricInterface* InOwner,
		const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	WORLDMETRICSCORE_API bool ReleaseExtension(
		UWorldMetricsExtension* InOwner,
		const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	template <class ExtensionClass UE_REQUIRES(std::is_base_of_v<UWorldMetricsExtension, ExtensionClass>)>
	bool ReleaseExtension(UWorldMetricInterface* InOwner)
	{
		return ReleaseExtension(InOwner, ExtensionClass::StaticClass());
	}

	template <class ExtensionClass UE_REQUIRES(std::is_base_of_v<UWorldMetricsExtension, ExtensionClass>)>
	bool ReleaseExtension(UWorldMetricsExtension* InExtensionOwner)
	{
		return ReleaseExtension(InExtensionOwner, ExtensionClass::StaticClass());
	}

private:
	static constexpr int32 DefaultExtensionCapacity = 16;
	static constexpr int32 DefaultOwnerListCapacity = 32;

	FTSTicker::FDelegateHandle UpdateTickerHandle;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UWorldMetricInterface>> Metrics;

	using FOwnerList = TSet<UObject*, DefaultKeyFuncs<UObject*>, TInlineSetAllocator<DefaultOwnerListCapacity>>;

	struct FExtension
	{
		TObjectPtr<UWorldMetricsExtension> Instance;
		FOwnerList Owners;
	};

	// Exposed to GC via AddReferencedObjects.
	TMap<TSubclassOf<UWorldMetricsExtension>, FExtension, TInlineSetAllocator<DefaultExtensionCapacity>> Extensions;

public:
	UPROPERTY(Config)
	float UpdateRateInSeconds = 0.f;

	/** The number of frames the subsystem waits to update added metrics after their initialization. */
	UPROPERTY(Config)
	int32 WarmUpFrames = 8;

private:
	int32 PendingWarmUpFrames = 0;

	void InitializeMetrics();
	void DeinitializeMetrics();

	/**
	 * Enables or disables the subsystem. When enabled the subsystem uses an update ticker to update each of the added
	 * world metrics. All metrics are automatically initialized when the system is enabled and deinitialized when is
	 * disabled. The subsystem is automatically disabled on deinitialization.
	 *
	 * @param bEnable: true for enabling the system and false otherwise.
	 */
	void Enable(bool bEnable);

	void OnUpdate(float DeltaTimeInSeconds);

	/**
	 * Removes all added metrics. The subsystem releases the hard-reference of the  metrics removed so they can be
	 * garbage-collected.
	 */
	void RemoveAllMetrics();

	/**
	 * Removes all existing metrics and extensions. Internal use only.
	 */
	void Clear();

	UWorldMetricsExtension* AcquireExtensionInternal(
		UObject* InOwner,
		const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	UWorldMetricsExtension* AcquireExistingExtension(
		UObject* InOwner,
		const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	UWorldMetricsExtension* AddExtension(UObject* InOwner, const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	bool ReleaseExtensionInternal(UObject* InOwner, const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	bool TryRemoveExtension(const TSubclassOf<UWorldMetricsExtension>& InExtensionClass);

	/**
	 * Verifies the parameter metric has released any acquired extensions, releasing any pending ones. It's the metric's
	 * responsibility to release all previously acquired resources. This method logs all extensions owned by the
	 * parameter metric that hasn't been released.
	 *
	 * @param InMetric: the metric to verify.
	 */
	void VerifyMetricReleasedAllExtensions(UWorldMetricInterface* InMetric);

	/**
	 * Verifies there are no orphan extensions. These are extensions that are still alive without any live metric. This
	 * method logs and removes all orphan extensions, addressing the scenario where extensions that acquired other
	 * extensions didn't release them. Extensions, like metrics, are responsible for releasing acquired extensions.
	 */
	void VerifyRemoveOrphanExtensions();
};
