// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS
#include "Engine/Engine.h"
#include "Tests/TestHarnessAdapter.h"
#endif	// WITH_TESTS

#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "WorldMetricCollection.h"
#include "WorldMetricsLog.h"
#include "WorldMetricsSubsystem.h"
#include "WorldMetricsTestTypes.h"
#include "WorldMetricsTestUtil.h"

//---------------------------------------------------------------------------------------------------------------------
// UE::WorldMetrics::Private
//---------------------------------------------------------------------------------------------------------------------
namespace UE::WorldMetrics::Private
{

void TestAll(UWorld* World);

static FAutoConsoleCommandWithWorldAndArgs CmdWorldMetricsSelfTest(
	TEXT("WorldMetrics.SelfTest"),
	TEXT("Toggles the World Metrics Subsystem self-test."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* World)
		{
			const UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
			if (!ensure(Subsystem))
			{
				UE_LOGF(LogWorldMetrics, Warning, "Cannot run SelfTest without a valid World Metrics subsystem.")
				return;
			}

			if (Subsystem->HasAnyMetric())
			{
				UE_LOGF(
					LogWorldMetrics, Warning,
					"Cannot run SelfTest in a World Metrics subsystem with pre-existing metrics.")
				return;
			}
			TestAll(World);
		}),
	ECVF_Default);

static const TSubclassOf<UMockWorldMetricBase> MockMetricClasses[] = {
	UMockWorldMetricA::StaticClass(), UMockWorldMetricB::StaticClass(), UMockWorldMetricC::StaticClass(),
	UMockWorldMetricD::StaticClass(), UMockWorldMetricE::StaticClass(), UMockWorldMetricF::StaticClass(),
	UMockWorldMetricG::StaticClass(), UMockWorldMetricH::StaticClass(), UMockWorldMetricI::StaticClass(),
	UMockWorldMetricJ::StaticClass()};
static constexpr int32 NumMockMetricClasses = sizeof(MockMetricClasses) / sizeof(MockMetricClasses[0]);

static const TSubclassOf<UMockWorldMetricsExtensionBase> MockExtensionClasses[] = {
	UMockWorldMetricsExtensionA::StaticClass(), UMockWorldMetricsExtensionB::StaticClass(),
	UMockWorldMetricsExtensionC::StaticClass(), UMockWorldMetricsExtensionD::StaticClass(),
	UMockWorldMetricsExtensionE::StaticClass(), UMockWorldMetricsExtensionF::StaticClass(),
	UMockWorldMetricsExtensionG::StaticClass(), UMockWorldMetricsExtensionH::StaticClass(),
	UMockWorldMetricsExtensionI::StaticClass(), UMockWorldMetricsExtensionJ::StaticClass()};

static constexpr int32 NumMockExtensionClasses = sizeof(MockExtensionClasses) / sizeof(MockExtensionClasses[0]);

TArray<TSubclassOf<UMockWorldMetricBase>> GetMockMetricClasses(UWorld* World, int32 Count, bool bRandomized = false)
{
	TArray<TSubclassOf<UMockWorldMetricBase>> Result;

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	if (UNLIKELY(!Subsystem))
	{
		return Result;
	}
	Count = FMath::Min(Count, NumMockMetricClasses);
	Result.Reserve(Count);

	const TArray<int32> ClassIndices =
		bRandomized ? MakeRandomSubset<int32>(MakeIndexArray(NumMockMetricClasses), Count) : MakeIndexArray(Count);

	for (int32 ClassIndex : ClassIndices)
	{
		Result.Emplace(MockMetricClasses[ClassIndex]);
	}
	return Result;
}

TArray<UMockWorldMetricBase*> AddMockMetrics(
	UWorld* World,
	TArrayView<TSubclassOf<UMockWorldMetricBase>> InMockMetricClasses)
{
	TArray<UMockWorldMetricBase*> Result;

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	if (UNLIKELY(!Subsystem))
	{
		return Result;
	}

	Result.Reserve(InMockMetricClasses.Num());
	for (const TSubclassOf<UMockWorldMetricBase>& MetricClass : InMockMetricClasses)
	{
		Result.Emplace(static_cast<UMockWorldMetricBase*>(Subsystem->AddMetric(MetricClass)));
	}
	return Result;
}

int32 RemoveMockMetrics(UWorld* World, TArrayView<UMockWorldMetricBase*> InMockMetrics, bool bRandomized = false)
{
	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	if (UNLIKELY(!Subsystem))
	{
		return 0;
	}

	TArray<int32> Indices = MakeIndexArray(InMockMetrics.Num());
	if (bRandomized)
	{
		Shuffle(Indices);
	}

	int32 Result = 0;
	for (int32 Index : Indices)
	{
		if (Subsystem->RemoveMetric(InMockMetrics[Index]))
		{
			++Result;
		}
	}
	return Result;
}

TArray<TSubclassOf<UMockWorldMetricsExtensionBase>>
AcquireMockMetricExtensions(UMockWorldMetricBase* Owner, int32 Count, bool bRandomized = false)
{
	TArray<TSubclassOf<UMockWorldMetricsExtensionBase>> Result;

	if (UNLIKELY(!Owner))
	{
		return Result;
	}

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(Owner->GetWorld());
	if (UNLIKELY(!Subsystem))
	{
		return Result;
	}

	Count = FMath::Min(Count, NumMockExtensionClasses);
	Result.Reserve(Count);

	const TArray<int32> ClassIndices =
		bRandomized ? MakeRandomSubset<int32>(MakeIndexArray(NumMockExtensionClasses), Count) : MakeIndexArray(Count);

	for (int32 ClassIndex : ClassIndices)
	{
		if (Subsystem->AcquireExtension(Owner, MockExtensionClasses[ClassIndex]))
		{
			Result.Emplace(MockExtensionClasses[ClassIndex]);
		}
	}
	return Result;
}

int32 ReleaseMockMetricExtensions(
	UMockWorldMetricBase* Owner,
	TArrayView<TSubclassOf<UMockWorldMetricsExtensionBase>> InMockExtensionClasses,
	bool bRandomized = false)
{
	if (UNLIKELY(!Owner))
	{
		return 0;
	}

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(Owner->GetWorld());
	if (UNLIKELY(!Subsystem))
	{
		return 0;
	}

	TArray<int32> ClassIndices = MakeIndexArray(InMockExtensionClasses.Num());
	if (bRandomized)
	{
		Shuffle(ClassIndices);
	}

	int32 Result = 0;
	for (int32 ClassIndex : ClassIndices)
	{
		if (Subsystem->ReleaseExtension(Owner, InMockExtensionClasses[ClassIndex]))
		{
			++Result;
		}
	}
	return Result;
}

void TestZeroState(UWorld* World)
{
	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem is valid after GetSubsystem."), Subsystem != nullptr);

	REQUIRE_MESSAGE(
		TEXT("WorldMetricsSubsystem shouldn't be enabled if there are no metrics."), !Subsystem->IsEnabled());
	REQUIRE_MESSAGE(TEXT("Unexpected existing metrics in WorldMetricsSubsystem"), !Subsystem->HasAnyMetric());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should zero metrics."), Subsystem->NumMetrics() == 0);

	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem shouldn't be have any extension."), !Subsystem->HasAnyExtension());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have zero extensions."), Subsystem->NumExtensions() == 0);
}

void TestSingleMetricAddRemove(UWorld* World)
{
	TestZeroState(World);

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);

	// Invalid metric add
	REQUIRE_MESSAGE(TEXT("AddMetric should fail"), !Subsystem->AddMetric(UObject::StaticClass()));

	UMockWorldMetricA* MetricA = Subsystem->AddMetric<UMockWorldMetricA>();
	REQUIRE_MESSAGE(TEXT("ContainsMetric failed"), Subsystem->ContainsMetric(MetricA));
	REQUIRE_MESSAGE(TEXT("Test Metric initialization failed"), MetricA->InitializeCount == 1);

	REQUIRE_MESSAGE(
		TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have metrics."), Subsystem->HasAnyMetric());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have one metric."), Subsystem->NumMetrics() == 1);

	// Same metric add
	REQUIRE_MESSAGE(TEXT("Unexpected AddMetric"), !Subsystem->AddMetric(MetricA));

	// Same metric class add
	UMockWorldMetricA* AnotherMetricA = Subsystem->AddMetric<UMockWorldMetricA>();
	REQUIRE_MESSAGE(TEXT("ContainsMetric failed"), Subsystem->ContainsMetric(AnotherMetricA));
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have one metric."), Subsystem->NumMetrics() == 2);

	// Metric remove
	REQUIRE_MESSAGE(TEXT("RemoveMetric failed."), Subsystem->RemoveMetric(MetricA));
	REQUIRE_MESSAGE(TEXT("Test Metric deinitialization failed after release."), MetricA->DeinitializeCount == 1);
	REQUIRE_MESSAGE(TEXT("Unexpected RemoveMetric."), !Subsystem->RemoveMetric(MetricA));

	REQUIRE_MESSAGE(TEXT("Unexpected RemoveMetric."), Subsystem->RemoveMetric(AnotherMetricA));
	REQUIRE_MESSAGE(TEXT("Test Metric deinitialization failed after release."), AnotherMetricA->DeinitializeCount == 1);
	REQUIRE_MESSAGE(TEXT("Unexpected RemoveMetric."), !Subsystem->RemoveMetric(AnotherMetricA));

	TestZeroState(World);
}

void TestMultipleMetricsAddRemove(UWorld* World, bool bRandomized)
{
	TestZeroState(World);

	constexpr int32 NumMetrics = 5;

	const UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);

	auto MetricClasses = Private::GetMockMetricClasses(World, NumMetrics, bRandomized);
	TArray<UMockWorldMetricBase*> MockMetrics = Private::AddMockMetrics(World, MetricClasses);
	REQUIRE_MESSAGE(TEXT("Unexpected empty metrics list."), !MockMetrics.IsEmpty());

	REQUIRE_MESSAGE(
		TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have metrics."), Subsystem->HasAnyMetric());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have one metric."), Subsystem->NumMetrics() == NumMetrics);

	for (const UMockWorldMetricBase* MockMetric : MockMetrics)
	{
		REQUIRE_MESSAGE(TEXT("Test Metric initialization failed"), MockMetric->InitializeCount == 1);
	}

	const int32 NumMetricsRemoved = Private::RemoveMockMetrics(World, MockMetrics, bRandomized);
	REQUIRE_MESSAGE(TEXT("Unexpected metric remove count"), NumMetricsRemoved == NumMetrics);
	for (const UMockWorldMetricBase* Metric : MockMetrics)
	{
		REQUIRE_MESSAGE(TEXT("Test Metric initialization failed"), Metric->DeinitializeCount == 1);
	}

	TestZeroState(World);
}

void TestSingleMetricSingleExtensionAcquireRelease(UWorld* World)
{
	TestZeroState(World);

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	UMockWorldMetricA* MetricA = Subsystem->AddMetric<UMockWorldMetricA>();
	REQUIRE_MESSAGE(TEXT("ContainsMetric failed"), Subsystem->ContainsMetric(MetricA));

	// Extension acquire/release
	{
		// Invalid metric acquire
		REQUIRE_MESSAGE(
			TEXT("AcquireExtension should fail"), !Subsystem->AcquireExtension(MetricA, UObject::StaticClass()));

		UMockWorldMetricsExtensionA* ExtensionA = Subsystem->AcquireExtension<UMockWorldMetricsExtensionA>(MetricA);
		REQUIRE_MESSAGE(TEXT("Acquire Test Extension failed."), ExtensionA != nullptr);
		REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->InitializeCount == 1);

		REQUIRE_MESSAGE(
			TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());
		REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have extensions."), Subsystem->HasAnyExtension());
		REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have one extension."), Subsystem->NumExtensions() == 1);

		// Invalid metric release
		REQUIRE_MESSAGE(
			TEXT("ReleaseExtension should fail"), !Subsystem->ReleaseExtension(MetricA, UObject::StaticClass()));

		REQUIRE_MESSAGE(
			TEXT("Test Extension release failed."), Subsystem->ReleaseExtension<UMockWorldMetricsExtensionA>(MetricA));
		REQUIRE_MESSAGE(
			TEXT("Test Extension deinitialization failed after release."), ExtensionA->DeinitializeCount == 1);
	}

	// Extension acquire/release through owner
	{
		UMockWorldMetricsExtensionA* ExtensionA =
			MetricA->GetOwner().AcquireExtension<UMockWorldMetricsExtensionA>(MetricA);
		REQUIRE_MESSAGE(TEXT("Acquire Test Extension failed."), ExtensionA != nullptr);
		REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->InitializeCount == 1);

		REQUIRE_MESSAGE(
			TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());
		REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have extensions."), Subsystem->HasAnyExtension());
		REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have one extension."), Subsystem->NumExtensions() == 1);

		REQUIRE_MESSAGE(
			TEXT("Test Extension release failed."),
			MetricA->GetOwner().ReleaseExtension<UMockWorldMetricsExtensionA>(MetricA));
		REQUIRE_MESSAGE(
			TEXT("Test Extension deinitialization failed after release."), ExtensionA->DeinitializeCount == 1);
	}

	Subsystem->RemoveMetric(MetricA);

	TestZeroState(World);
}

void TestMultipleMetricsSingleExtensionAcquireReleaseChildExtension(UWorld* World)
{
	TestZeroState(World);

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);

	UMockWorldMetricA* MetricA = Subsystem->AddMetric<UMockWorldMetricA>();
	REQUIRE_MESSAGE(TEXT("ContainsMetric failed"), Subsystem->ContainsMetric(MetricA));

	UMockWorldMetricB* MetricB = Subsystem->AddMetric<UMockWorldMetricB>();
	REQUIRE_MESSAGE(TEXT("ContainsMetric failed"), Subsystem->ContainsMetric(MetricB));

	REQUIRE_MESSAGE(
		TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());

	const UMockWorldMetricsExtensionBase* ExtensionA = nullptr;
	const UMockWorldMetricsExtensionBase* ExtensionB = nullptr;
	const UMockWorldMetricsExtensionBase* ExtensionC = nullptr;

	// Create lambda to handle acquiring/releasing other extensions when an extension is initialized/deinitialized.
	auto ExtensionInitializeDeinitializeHandler = [&ExtensionC](UWorldMetricsExtension* Extension, bool bInitialize)
	{
		// B acquires C.
		if (Extension->IsA<UMockWorldMetricsExtensionB>())
		{
			if (bInitialize)
			{
				ExtensionC = Extension->GetOwner().AcquireExtension<UMockWorldMetricsExtensionC>(Extension);
			}
			else
			{
				Extension->GetOwner().ReleaseExtension<UMockWorldMetricsExtensionC>(Extension);
			}
		}
	};

	// TGuardValue to make sure the static OnInitializeDeinitialize delegate is unbound on scope exit.
	TGuardValue InitializeDeinitializeScope(
		UMockWorldMetricsExtensionBase::OnInitializeDeinitialize,
		UMockWorldMetricsExtensionBase::FInitializeDeinitializeDelegate::CreateLambda(ExtensionInitializeDeinitializeHandler));

	// Acquire Extension A for Metric A.
	{
		ExtensionA = MetricA->GetOwner().AcquireExtension<UMockWorldMetricsExtensionA>(MetricA);
		REQUIRE_MESSAGE(TEXT("Acquire Test Extension A failed."), ExtensionA != nullptr);
		REQUIRE_MESSAGE(TEXT("Test Extension A initialization failed"), ExtensionA->InitializeCount == 1);

		REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have extensions."), Subsystem->HasAnyExtension());
		REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have 1 extension."), Subsystem->NumExtensions() == 1);
	}

	// Acquire Extension B (and C, transitively) for Metric B.
	{
		ExtensionB = MetricB->GetOwner().AcquireExtension<UMockWorldMetricsExtensionB>(MetricB);
		REQUIRE_MESSAGE(TEXT("Acquire Test Extension B failed."), ExtensionB != nullptr);
		REQUIRE_MESSAGE(TEXT("Test Extension B initialization failed"), ExtensionB->InitializeCount == 1);

		REQUIRE_MESSAGE(TEXT("Acquire Test Extension C failed."), ExtensionC != nullptr);
		REQUIRE_MESSAGE(TEXT("Test Extension C initialization failed"), ExtensionC->InitializeCount == 1);

		REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have extensions."), Subsystem->HasAnyExtension());
		REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have 3 extensions."), Subsystem->NumExtensions() == 3);
	}

	// Release extension for Metric A.
	{
		REQUIRE_MESSAGE(
			TEXT("Test Extension release failed."),
			MetricA->GetOwner().ReleaseExtension<UMockWorldMetricsExtensionA>(MetricA));
		REQUIRE_MESSAGE(
			TEXT("Test Extension deinitialization failed after release."), ExtensionA->DeinitializeCount == 1);
	}

	// Release extension for Metric B.
	{
		REQUIRE_MESSAGE(
			TEXT("Test Extension B release failed."),
			MetricB->GetOwner().ReleaseExtension<UMockWorldMetricsExtensionB>(MetricB));
		REQUIRE_MESSAGE(
			TEXT("Test Extension B deinitialization failed after release."), ExtensionB->DeinitializeCount == 1);
		REQUIRE_MESSAGE(
			TEXT("Test Extension C deinitialization failed after release."), ExtensionC->DeinitializeCount == 1);
	}

	Subsystem->RemoveMetric(MetricB);
	Subsystem->RemoveMetric(MetricA);

	TestZeroState(World);
}

void TestMultipleMetricsSingleExtensionAcquireRelease(UWorld* World, bool bRandomized)
{
	TestZeroState(World);

	constexpr int32 NumMetrics = 5;

	auto MetricClasses = Private::GetMockMetricClasses(World, NumMetrics, bRandomized);
	TArray<UMockWorldMetricBase*> MockMetrics = Private::AddMockMetrics(World, MetricClasses);
	REQUIRE_MESSAGE(TEXT("Unexpected empty metrics list."), !MockMetrics.IsEmpty());

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	UMockWorldMetricsExtensionA* ExtensionA = Subsystem->AcquireExtension<UMockWorldMetricsExtensionA>(MockMetrics[0]);

	// Acquire extension loop
	{
		int32 CheckCount = 1;
		for (UMockWorldMetricBase* MockMetric : MockMetrics)
		{
			Subsystem->AcquireExtension<UMockWorldMetricsExtensionA>(MockMetric);
			REQUIRE_MESSAGE(TEXT("Acquire Test Extension failed."), ExtensionA != nullptr);
			REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->InitializeCount == 1);
			REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->OnAcquireCount == ++CheckCount);
		}
	}
	// Release extension loop
	{
		if (bRandomized)
		{
			Shuffle(MockMetrics);
		}

		int32 CheckCount = 0;
		for (UMockWorldMetricBase* MockMetric : MockMetrics)
		{
			REQUIRE_MESSAGE(TEXT("Test Extension unexpected deinitialization"), ExtensionA->DeinitializeCount == 0);
			Subsystem->ReleaseExtension<UMockWorldMetricsExtensionA>(MockMetric);

			REQUIRE_MESSAGE(TEXT("Acquire Test Extension failed."), ExtensionA != nullptr);
			REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->InitializeCount == 1);
			REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->OnReleaseCount == ++CheckCount);
		}
		REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->DeinitializeCount == 1);
	}

	Private::RemoveMockMetrics(World, MockMetrics, bRandomized);

	TestZeroState(World);
}

void TestMultipleMetricsMultipleExtensionAcquireRelease(UWorld* World, bool bRandomized)
{
	TestZeroState(World);

	constexpr int32 NumMetrics = 5;
	constexpr int32 NumExtension = 5;

	auto MetricClasses = Private::GetMockMetricClasses(World, NumMetrics, bRandomized);
	TArray<UMockWorldMetricBase*> MockMetrics = Private::AddMockMetrics(World, MetricClasses);
	REQUIRE_MESSAGE(TEXT("Unexpected empty metrics list."), !MockMetrics.IsEmpty());

	TArray<TArray<TSubclassOf<UMockWorldMetricsExtensionBase>>> Extensions;
	Extensions.Reserve(NumMetrics);

	// Acquire extensions
	for (UMockWorldMetricBase* MockMetric : MockMetrics)
	{
		Extensions.Emplace(Private::AcquireMockMetricExtensions(MockMetric, NumExtension, bRandomized));
	}

	const UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	REQUIRE_MESSAGE(
		TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have extensions."), Subsystem->HasAnyExtension());

	// Release extensions
	for (int32 MetricIndex = 0; MetricIndex < NumMetrics; ++MetricIndex)
	{
		Private::ReleaseMockMetricExtensions(MockMetrics[MetricIndex], Extensions[MetricIndex], bRandomized);
	}

	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem shouldn't be have any extension."), !Subsystem->HasAnyExtension());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have zero extensions."), Subsystem->NumExtensions() == 0);

	Private::RemoveMockMetrics(World, MockMetrics, bRandomized);

	TestZeroState(World);
}

void TestSingleMetricSingleExtensionAutoReleaseOnRemoval(UWorld* World)
{
	TestZeroState(World);

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	UMockWorldMetricA* MetricA = Subsystem->AddMetric<UMockWorldMetricA>();
	REQUIRE_MESSAGE(TEXT("ContainsMetric failed"), Subsystem->ContainsMetric(MetricA));

	UMockWorldMetricsExtensionA* ExtensionA = Subsystem->AcquireExtension<UMockWorldMetricsExtensionA>(MetricA);
	REQUIRE_MESSAGE(TEXT("Acquire Test Extension failed."), ExtensionA != nullptr);
	REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->InitializeCount == 1);

	REQUIRE_MESSAGE(
		TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have extensions."), Subsystem->HasAnyExtension());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have one extension."), Subsystem->NumExtensions() == 1);

	Subsystem->RemoveMetric(MetricA);

	REQUIRE_MESSAGE(TEXT("Test Extension deinitialization failed after release."), ExtensionA->DeinitializeCount == 1);
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem shouldn't have any extensions."), !Subsystem->HasAnyExtension());

	TestZeroState(World);
}

void TestMultipleMetricsMultipleExtensionAutoReleaseOnRemoval(UWorld* World, bool bRandomized)
{
	TestZeroState(World);

	constexpr int32 NumMetrics = 5;
	constexpr int32 NumExtension = 5;

	auto MetricClasses = Private::GetMockMetricClasses(World, NumMetrics, bRandomized);
	TArray<UMockWorldMetricBase*> MockMetrics = Private::AddMockMetrics(World, MetricClasses);
	REQUIRE_MESSAGE(TEXT("Unexpected empty metrics list."), !MockMetrics.IsEmpty());

	// Acquire extensions
	for (UMockWorldMetricBase* MockMetric : MockMetrics)
	{
		Private::AcquireMockMetricExtensions(MockMetric, NumExtension, bRandomized);
	}

	const UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	REQUIRE_MESSAGE(
		TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have extensions."), Subsystem->HasAnyExtension());

	Private::RemoveMockMetrics(World, MockMetrics, bRandomized);

	TestZeroState(World);
}

void TestSingleMetricOrphanExtensionAutoRemoval(UWorld* World)
{
	TestZeroState(World);

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
	UMockWorldMetricA* MetricA = Subsystem->AddMetric<UMockWorldMetricA>();
	REQUIRE_MESSAGE(TEXT("ContainsMetric failed"), Subsystem->ContainsMetric(MetricA));

	UMockWorldMetricsExtensionA* ExtensionA = Subsystem->AcquireExtension<UMockWorldMetricsExtensionA>(MetricA);
	REQUIRE_MESSAGE(TEXT("Acquire Test Extension failed."), ExtensionA != nullptr);
	REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionA->InitializeCount == 1);

	// Extension dependency: ExtensionB requires ExtensionA
	UMockWorldMetricsExtensionB* ExtensionB = Subsystem->AcquireExtension<UMockWorldMetricsExtensionB>(ExtensionA);
	REQUIRE_MESSAGE(TEXT("Acquire Test Extension failed."), ExtensionB != nullptr);
	REQUIRE_MESSAGE(TEXT("Test Extension initialization failed"), ExtensionB->InitializeCount == 1);

	REQUIRE_MESSAGE(
		TEXT("WorldMetricsSubsystem should be enabled by default if there are metrics."), Subsystem->IsEnabled());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have extensions."), Subsystem->HasAnyExtension());
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem should have one extension."), Subsystem->NumExtensions() == 2);

	Subsystem->RemoveMetric(MetricA);

	REQUIRE_MESSAGE(TEXT("Test Extension deinitialization failed after release."), ExtensionA->DeinitializeCount == 1);
	REQUIRE_MESSAGE(TEXT("Test Extension deinitialization failed after release."), ExtensionB->DeinitializeCount == 1);
	REQUIRE_MESSAGE(TEXT("WorldMetricsSubsystem shouldn't have any extensions."), !Subsystem->HasAnyExtension());

	TestZeroState(World);
}

void TestMultipleMetricIteration(UWorld* World, bool bRandomized)
{
	TestZeroState(World);

	constexpr int32 NumMetrics = 8;

	auto MetricClasses = Private::GetMockMetricClasses(World, NumMetrics, bRandomized);
	TArray<UMockWorldMetricBase*> MockMetrics = Private::AddMockMetrics(World, MetricClasses);
	REQUIRE_MESSAGE(TEXT("Unexpected empty metrics list."), !MockMetrics.IsEmpty());

	// const iteration
	{
		const UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);
		int32 CheckCount = 0;
		int32 FooClassCount = 0;
		int32 BarClassCount = 0;
		Subsystem->ForEachMetric(
			[&](const UWorldMetricInterface* Metric)
			{
				++CheckCount;
				if (Metric->IsA<UMockWorldMetricFooBase>())
				{
					++FooClassCount;
				}
				if (Metric->IsA<UMockWorldMetricBarBase>())
				{
					++BarClassCount;
				}
				return true;
			});
		REQUIRE_MESSAGE(TEXT("Check count failed"), CheckCount == NumMetrics);

		CheckCount = 0;
		Subsystem->ForEachMetricOfClass<UMockWorldMetricFooBase>(
			[&](const UMockWorldMetricFooBase* MockMetric)
			{
				++CheckCount;
				return true;
			});
		REQUIRE_MESSAGE(TEXT("Check count failed"), CheckCount == FooClassCount);

		CheckCount = 0;
		Subsystem->ForEachMetricOfClass<UMockWorldMetricBarBase>(
			[&](const UMockWorldMetricBarBase* MockMetric)
			{
				++CheckCount;
				return true;
			});
		REQUIRE_MESSAGE(TEXT("Check count failed"), CheckCount == BarClassCount);
	}

	Private::RemoveMockMetrics(World, MockMetrics, bRandomized);

	TestZeroState(World);
}

void TestMetricCollection(UWorld* World, bool bRandomized)
{
	TestZeroState(World);

	constexpr int32 NumMetrics = 8;

	UWorldMetricsSubsystem* Subsystem = UWorldMetricsSubsystem::Get(World);

	TArray<TSubclassOf<UMockWorldMetricBase>> MetricClasses = Private::GetMockMetricClasses(World, NumMetrics, bRandomized);

	FWorldMetricCollection Collection;
	Collection.Initialize(World);

	// Add
	for (const TSubclassOf<UMockWorldMetricBase>& MetricClass : MetricClasses)
	{
		REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'Add' failure."), Collection.Add(MetricClass));
	}

	REQUIRE_MESSAGE(TEXT("Unexpected empty metrics list."), !Collection.IsEmpty());
	REQUIRE_MESSAGE(TEXT("Unexpected metrics count mismatch."), Collection.Num() == MetricClasses.Num());

	// Contains/Get
	for (const TSubclassOf<UMockWorldMetricBase>& MetricClass : MetricClasses)
	{
		REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'Contains' failure."), Collection.Contains(MetricClass));
		REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'Get' failure."), Collection.Get(MetricClass) != nullptr);
	}

	// Iteration (const)
	{
		int32 CheckCount = 0;
		int32 FooClassCount = 0;
		int32 BarClassCount = 0;
		Collection.ForEach(
			[&](const UWorldMetricInterface* Metric)
			{
				++CheckCount;
				if (Metric->IsA<UMockWorldMetricFooBase>())
				{
					++FooClassCount;
				}
				if (Metric->IsA<UMockWorldMetricBarBase>())
				{
					++BarClassCount;
				}
				return true;
			});
		REQUIRE_MESSAGE(TEXT("Check count failed"), CheckCount == NumMetrics);

		CheckCount = 0;
		Collection.ForEach<UMockWorldMetricFooBase>(
			[&](const UMockWorldMetricFooBase* MockMetric)
			{
				++CheckCount;
				return true;
			});
		REQUIRE_MESSAGE(TEXT("Check count failed"), CheckCount == FooClassCount);

		CheckCount = 0;
		Collection.ForEach<UMockWorldMetricBarBase>(
			[&](const UMockWorldMetricBarBase* MockMetric)
			{
				++CheckCount;
				return true;
			});
		REQUIRE_MESSAGE(TEXT("Check count failed"), CheckCount == BarClassCount);
	}

	// Enable state
	REQUIRE_MESSAGE(TEXT("Unexpected existing metrics in WorldMetricsSubsystem"), !Subsystem->HasAnyMetric());
	Collection.Enable(true);
	REQUIRE_MESSAGE(TEXT("Unexpected non-existing metrics in WorldMetricsSubsystem"), Subsystem->HasAnyMetric());
	REQUIRE_MESSAGE(TEXT("Unexpected number of metrics in WorldMetricsSubsystem."), Subsystem->NumMetrics() == NumMetrics);
	Collection.Enable(false);
	REQUIRE_MESSAGE(TEXT("Unexpected existing metrics in WorldMetricsSubsystem"), !Subsystem->HasAnyMetric());

	// Remove
	for (const TSubclassOf<UMockWorldMetricBase>& MetricClass : MetricClasses)
	{
		REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'Remove' failure."), Collection.Remove(MetricClass));
	}

	REQUIRE_MESSAGE(TEXT("Unexpected populated metrics collection."), Collection.IsEmpty());

	// Add
	REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'Add<UMockWorldMetricA>' failure."), Collection.Add<UMockWorldMetricA>());

	// Contains
	REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'Contains<UMockWorldMetricA>' failure."), Collection.Contains<UMockWorldMetricA>());

	// Get
	REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'Get<UMockWorldMetricA>' failure."), Collection.Get<UMockWorldMetricA>() != nullptr);

	// Remove
	REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'Remove<UMockWorldMetricA>' failure."), Collection.Remove<UMockWorldMetricA>());

	// GetOrAdd
	UMockWorldMetricA* MetricA = Collection.GetOrAdd<UMockWorldMetricA>();
	REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'GetOrAdd<UMockWorldMetricA>' failure."), MetricA != nullptr);	
	UMockWorldMetricA* MetricB = Collection.GetOrAdd<UMockWorldMetricA>();
	REQUIRE_MESSAGE(TEXT("Unexpected metric collection 'GetOrAdd<UMockWorldMetricA>' failure."), MetricB != nullptr);
	REQUIRE_MESSAGE(TEXT("Unexpected different metric instances."), MetricA == MetricB);

	Collection.Reset();

	TestZeroState(World);
}

void TestAll(UWorld* World)
{
	TestZeroState(World);

	// Metrics
	TestSingleMetricAddRemove(World);
	TestMultipleMetricsAddRemove(World, false);
	TestMultipleMetricsAddRemove(World, true);
	TestMultipleMetricIteration(World, false);
	TestMultipleMetricIteration(World, true);

	// Extension
	TestSingleMetricSingleExtensionAcquireRelease(World);
	TestMultipleMetricsSingleExtensionAcquireRelease(World, false);
	TestMultipleMetricsSingleExtensionAcquireRelease(World, true);
	TestMultipleMetricsSingleExtensionAcquireReleaseChildExtension(World);
	TestMultipleMetricsMultipleExtensionAcquireRelease(World, false);
	TestMultipleMetricsMultipleExtensionAcquireRelease(World, true);

	// Extension auto-removal when no longer acquired
	TestSingleMetricSingleExtensionAutoReleaseOnRemoval(World);
	TestMultipleMetricsMultipleExtensionAutoReleaseOnRemoval(World, false);
	TestMultipleMetricsMultipleExtensionAutoReleaseOnRemoval(World, true);
	TestSingleMetricOrphanExtensionAutoRemoval(World);

	// Collection
	TestMetricCollection(World, true);
	TestMetricCollection(World, false);
}

}  // namespace UE::WorldMetrics::Private

//---------------------------------------------------------------------------------------------------------------------
// WITH_TESTS
//---------------------------------------------------------------------------------------------------------------------

#if WITH_TESTS
namespace UE::WorldMetrics
{
namespace Private
{
static void ScopedWorldTest(EWorldType::Type WorldType, TFunctionRef<void(UWorld* World)> WorldTest)
{
	REQUIRE_MESSAGE(TEXT("GEngine is required for this test"), GEngine != nullptr);
	
	constexpr bool bInformEngineOfWorld = false;
	UWorld* World = UWorld::CreateWorld(WorldType, bInformEngineOfWorld);
	REQUIRE_MESSAGE(TEXT("World could not be created."), World != nullptr);

	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(WorldType);
	WorldContext.SetCurrentWorld(World);

	const FURL URL;
	World->InitializeActorsForPlay(URL);
	World->BeginPlay();

	WorldTest(World);

	World->EndPlay(EEndPlayReason::Quit);
	GEngine->DestroyWorldContext(World);
	World->DestroyWorld(bInformEngineOfWorld);
}
}  // namespace Private

TEST_CASE_NAMED(WorldMetricsTestZeroState, "WorldMetrics::TestZeroState", "[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(EWorldType::Editor, [](UWorld* World) { Private::TestZeroState(World); });
}

TEST_CASE_NAMED(
	WorldMetricsTestSingleMetricAddRemove,
	"WorldMetrics::TestSingleMetricAddRemove",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(EWorldType::Editor, [](UWorld* World) { Private::TestSingleMetricAddRemove(World); });
}

TEST_CASE_NAMED(
	WorldMetricsTestMultipleMetricsAddRemove,
	"WorldMetrics::TestMultipleMetricsAddRemove",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor,
		[](UWorld* World)
		{
			Private::TestMultipleMetricsAddRemove(World, false);
			Private::TestMultipleMetricsAddRemove(World, true);
		});
}

TEST_CASE_NAMED(
	WorldMetricsTestMultipleMetricIteration,
	"WorldMetrics::TestMultipleMetricIteration",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor,
		[](UWorld* World)
		{
			Private::TestMultipleMetricIteration(World, false);
			Private::TestMultipleMetricIteration(World, true);
		});
}

TEST_CASE_NAMED(
	WorldMetricsTestSingleMetricSingleExtensionAcquireRelease,
	"WorldMetrics::TestSingleMetricSingleExtensionAcquireRelease",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor, [](UWorld* World) { Private::TestSingleMetricSingleExtensionAcquireRelease(World); });
}

TEST_CASE_NAMED(
	WorldMetricsTestMultipleMetricsSingleExtensionAcquireRelease,
	"WorldMetrics::TestMultipleMetricsSingleExtensionAcquireRelease",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor,
		[](UWorld* World)
		{
			Private::TestMultipleMetricsSingleExtensionAcquireRelease(World, false);
			Private::TestMultipleMetricsSingleExtensionAcquireRelease(World, true);
		});
}

TEST_CASE_NAMED(
	WorldMetricsTestMultipleMetricsSingleExtensionAcquireReleaseChildExtension,
	"WorldMetrics::TestMultipleMetricsSingleExtensionAcquireReleaseChildExtension",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor,
		[](UWorld* World)
		{
			Private::TestMultipleMetricsSingleExtensionAcquireReleaseChildExtension(World);
		});
}

TEST_CASE_NAMED(
	WorldMetricsTestMultipleMetricsMultipleExtensionAcquireRelease,
	"WorldMetrics::TestMultipleMetricsMultipleExtensionAcquireRelease",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor,
		[](UWorld* World)
		{
			Private::TestMultipleMetricsMultipleExtensionAcquireRelease(World, false);
			Private::TestMultipleMetricsMultipleExtensionAcquireRelease(World, true);
		});
}

TEST_CASE_NAMED(
	WorldMetricsTestSingleMetricSingleExtensionAutoReleaseOnRemoval,
	"WorldMetrics::TestSingleMetricSingleExtensionAutoReleaseOnRemoval",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor,
		[](UWorld* World) { Private::TestSingleMetricSingleExtensionAutoReleaseOnRemoval(World); });
}

TEST_CASE_NAMED(
	WorldMetricsTestMultipleMetricsMultipleExtensionAutoReleaseOnRemoval,
	"WorldMetrics::TestMultipleMetricsMultipleExtensionAutoReleaseOnRemoval",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor,
		[](UWorld* World)
		{
			Private::TestMultipleMetricsMultipleExtensionAutoReleaseOnRemoval(World, false);
			Private::TestMultipleMetricsMultipleExtensionAutoReleaseOnRemoval(World, true);
		});
}

TEST_CASE_NAMED(
	WorldMetricsTestSingleMetricOrphanExtensionAutoRemoval,
	"WorldMetrics::TestSingleMetricOrphanExtensionAutoRemoval",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor, [](UWorld* World) { Private::TestSingleMetricOrphanExtensionAutoRemoval(World); });
}

TEST_CASE_NAMED(
	TestMetricCollection,
	"WorldMetrics::TestMetricCollection",
	"[WorldMetrics][Core][EngineFilter]")
{
	Private::ScopedWorldTest(
		EWorldType::Editor,
		[](UWorld* World)
		{
			Private::TestMetricCollection(World, true);
			Private::TestMetricCollection(World, false);
		});
}


}  // namespace UE::WorldMetrics

#endif	// WITH_TESTS
