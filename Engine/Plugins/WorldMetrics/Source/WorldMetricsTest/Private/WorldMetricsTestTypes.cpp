// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldMetricsTestTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldMetricsTestTypes)

//---------------------------------------------------------------------------------------------------------------------
// UMockWorldMetricBase
//---------------------------------------------------------------------------------------------------------------------

void UMockWorldMetricBase::Initialize()
{
	++InitializeCount;
}

void UMockWorldMetricBase::Deinitialize()
{
	++DeinitializeCount;
}

void UMockWorldMetricBase::Update(float /*DeltaTimeInSeconds*/)
{
	++UpdateCount;
}

//---------------------------------------------------------------------------------------------------------------------
// UMockWorldMetricsExtensionBase
//---------------------------------------------------------------------------------------------------------------------

UMockWorldMetricsExtensionBase::FInitializeDeinitializeDelegate UMockWorldMetricsExtensionBase::OnInitializeDeinitialize;

void UMockWorldMetricsExtensionBase::Initialize()
{
	++InitializeCount;
	OnInitializeDeinitialize.ExecuteIfBound(this, true);
}

void UMockWorldMetricsExtensionBase::Deinitialize()
{
	++DeinitializeCount;
	OnInitializeDeinitialize.ExecuteIfBound(this, false);
}

void UMockWorldMetricsExtensionBase::OnAcquire(UObject* InOwner)
{
	++OnAcquireCount;
}

void UMockWorldMetricsExtensionBase::OnRelease(UObject* InOwner)
{
	++OnReleaseCount;
}
