// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldMetricInterface.h"
#include "WorldMetricsExtension.h"

#include "WorldMetricsTestTypes.generated.h"

UCLASS(abstract)
class UMockWorldMetricBase : public UWorldMetricInterface
{
	GENERATED_BODY()

	virtual void Initialize() override;
	virtual void Deinitialize() override;
	virtual void Update(float DeltaTimeInSeconds) override;

public:
	TArray<UWorldMetricsExtension*> Extensions;
	int32 InitializeCount = 0;
	int32 DeinitializeCount = 0;
	int32 UpdateCount = 0;

	virtual SIZE_T GetAllocatedSize() const override
	{
		return 0;
	}
};

UCLASS(abstract)
class UMockWorldMetricsExtensionBase : public UWorldMetricsExtension
{
	GENERATED_BODY()

	virtual void Initialize() override;
	virtual void Deinitialize() override;
	virtual void OnAcquire(UObject* InOwner) override;
	virtual void OnRelease(UObject* InOwner) override;

public:
	int32 InitializeCount = 0;
	int32 DeinitializeCount = 0;
	int32 OnAcquireCount = 0;
	int32 OnReleaseCount = 0;

	// Static delegate to allow injecting logic into Initialize/Deinitialize without needing to create new classes.
	using FInitializeDeinitializeDelegate = TDelegate<void(UWorldMetricsExtension* Extension, bool bInitialize)>;
	static FInitializeDeinitializeDelegate OnInitializeDeinitialize;

	virtual SIZE_T GetAllocatedSize() const override
	{
		return 0;
	}
};

UCLASS()
class UMockWorldMetricFooBase: public UMockWorldMetricBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricBarBase : public UMockWorldMetricBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricA : public UMockWorldMetricFooBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricB : public UMockWorldMetricFooBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricC : public UMockWorldMetricFooBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricD : public UMockWorldMetricFooBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricE : public UMockWorldMetricFooBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricF : public UMockWorldMetricBarBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricG : public UMockWorldMetricBarBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricH : public UMockWorldMetricBarBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricI : public UMockWorldMetricBarBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricJ : public UMockWorldMetricBarBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionA : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionB : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionC : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionD : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionE : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionF : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionG : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionH : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionI : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};

UCLASS()
class UMockWorldMetricsExtensionJ : public UMockWorldMetricsExtensionBase
{
	GENERATED_BODY()
};
