// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#ifndef REQUIRE_MESSAGE

#include "WorldMetricsLog.h"

#define REQUIRE_MESSAGE(Message, Expr)           \
	if (!(Expr))                                 \
	{                                            \
		UE_LOG(LogWorldMetrics, Error, Message); \
		return;                                  \
	}
#endif // REQUIRE_MESSAGE

namespace UE::WorldMetrics
{

/**
 * Returns an array with the parameter number of sequentially increased index values starting from zero.
 * This method is equivalent to using std::iota with a zero starting value in a sequential container.
 */
inline TArray<int32> MakeIndexArray(int32 Size)
{
	// TODO: consider using std::iota when <algorithm> is allowed
	TArray<int32> Indices;
	Indices.SetNum(Size);
	for (int32 i = 0; i < Size; ++i)
	{
		Indices[i] = i;
	}
	return Indices;
}

/**
 * Returns an array with a subset of the elements in the parameter input array. The subset size is expressed
 * by the parameter size which must be smaller than the number of elements in the input array.
 */
template <typename T>
TArray<T> MakeRandomSubset(TArrayView<T> Items, int32 Size = 0)
{
	if (!Size)
	{
		Size = FMath::Max(1, Items.Num() / 2);
	}
	if (Items.Num() == Size)
	{
		return TArray<T>(Items);
	}
	TArray<T> Result;
	if (Items.Num() > 1 && Size < Items.Num())
	{
		Result.Reset(Size);
		while (Result.Num() < Size)
		{
			const int32 i = FMath::RandRange(0, Items.Num() - 1);
			if (Result.Find(Items[i]) == INDEX_NONE)
			{
				Result.Add(Items[i]);
			}
		}
	}
	return Result;
}

/**
 * Performs a number of randomized swaps in the parameter array equivalent to its number of elements.
 */
template <typename T>
void Shuffle(TArray<T>& OutArray)
{
	if (OutArray.Num() > 1)
	{
		for (int32 i = 0; i < OutArray.Num(); i++)
		{
			const int32 j = FMath::RandRange(i, OutArray.Num() - 1);
			if (i != j)
			{
				OutArray.Swap(i, j);
			}
		}
	}
}

}  // namespace UE::WorldMetrics
