{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "World Metrics",
	"Description": "Provides access to the world metrics system",
	"Category": "Profiling",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"EnabledByDefault": true,
	"Installed": false,
	"EditorCustomVirtualPath": "WorldMetrics",
	"SupportedPrograms": [
		"SpatialMetricsProfiler"
	],
	"Modules": [
		{
			"Name": "WorldMetricsCore",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		},
		{
			"Name": "WorldMetricsTest",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default",
			"TargetConfigurationDenyList": [ "Shipping" ]
		},
		{
			"Name": "CsvMetrics",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		}
	]
}
